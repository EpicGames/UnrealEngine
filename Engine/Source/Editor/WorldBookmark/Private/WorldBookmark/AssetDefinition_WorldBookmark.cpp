// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldBookmark/AssetDefinition_WorldBookmark.h"
#include "WorldBookmark/WorldBookmark.h"
#include "WorldBookmark/WorldBookmarkCommands.h"
#include "WorldBookmark/WorldBookmarkStyle.h"
#include "ContentBrowserMenuContexts.h"
#include "Toolkits/SimpleAssetEditor.h"
#include "ToolMenus.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_WorldBookmark)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

EAssetCommandResult UAssetDefinition_WorldBookmark::ActivateAssets(const FAssetActivateArgs& ActivateArgs) const
{
	bOpenFromContentBrowser = true;
	return Super::ActivateAssets(ActivateArgs);
}

EAssetCommandResult UAssetDefinition_WorldBookmark::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	const TArray<UWorldBookmark*> WorldBookmarks = OpenArgs.LoadObjects<UWorldBookmark>();
	check(WorldBookmarks.Num() > 0);

	bool bEditAsset = bOpenFromContentBrowser;
	bOpenFromContentBrowser = false;

	if (bEditAsset)
	{
		for (UWorldBookmark* WorldBookmark : WorldBookmarks)
		{
			FSimpleAssetEditor::CreateEditor(EToolkitMode::Standalone, OpenArgs.ToolkitHost, WorldBookmark);
		}
	}
	else
	{
		UWorldBookmark* WorldBookmark = WorldBookmarks[0];
		check(WorldBookmark != nullptr);
		WorldBookmark->Load();
	}

	return EAssetCommandResult::Handled;
}


//--------------------------------------------------------------------
// Menu Extensions
//--------------------------------------------------------------------
namespace MenuExtension_Bookmark
{	
	UWorldBookmark* GetBookmarkFromContext(const FToolMenuContext& InContext)
	{
		if (UContentBrowserAssetContextMenuContext::GetNumAssetsSelected(InContext) != 1)
		{
			return nullptr;
		}

		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		const TArray<UWorldBookmark*> WorldBookmarks = Context->LoadSelectedObjects<UWorldBookmark>();
		return WorldBookmarks.Num() == 1 ? WorldBookmarks[0] : nullptr;
	}

	bool CanExecuteGoToWorldBookmark(const FToolMenuContext& InContext)
	{
		UWorldBookmark* WorldBookmark = GetBookmarkFromContext(InContext);
		return WorldBookmark && WorldBookmark->CanLoad();
	}

	bool CanExecuteUpdateWorldBookmark(const FToolMenuContext& InContext)
	{
		UWorldBookmark* WorldBookmark = GetBookmarkFromContext(InContext);
		return WorldBookmark && WorldBookmark->CanUpdate();
	}

	void ExecuteGoToWorldBookmark(const FToolMenuContext& InContext)
	{
		if (UWorldBookmark* WorldBookmark = GetBookmarkFromContext(InContext))
		{
			WorldBookmark->Load();
		}
	}

	void ExecuteUpdateWorldBookmark(const FToolMenuContext& InContext)
	{
		if (UWorldBookmark* WorldBookmark = GetBookmarkFromContext(InContext))
		{
			WorldBookmark->Update();
		}
	}

	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []
	{
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UWorldBookmark::StaticClass());

			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				auto AddMenuEntry = [&InSection](TSharedPtr<FUICommandInfo> InCommandInfo, FToolUIAction InUIAction)
				{
					InSection.AddMenuEntry(InCommandInfo->GetCommandName(), InCommandInfo->GetLabel(), InCommandInfo->GetDescription(), InCommandInfo->GetIcon(), InUIAction);
				};

				FToolUIAction UIAction;

				UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteGoToWorldBookmark);
				UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&CanExecuteGoToWorldBookmark);
				AddMenuEntry(FWorldBookmarkCommands::Get().LoadBookmark, UIAction);

				UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteUpdateWorldBookmark);
				UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&CanExecuteUpdateWorldBookmark);
				AddMenuEntry(FWorldBookmarkCommands::Get().UpdateBookmark, UIAction);
			}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
