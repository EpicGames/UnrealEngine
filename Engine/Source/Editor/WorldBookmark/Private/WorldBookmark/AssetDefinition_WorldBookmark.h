// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldBookmark/WorldBookmark.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_WorldBookmark.generated.h"


UCLASS()
class UAssetDefinition_WorldBookmark : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_WorldBookmark", "Bookmark"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(255, 0, 0)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UWorldBookmark::StaticClass(); }
	virtual FAssetSupportResponse CanLocalize(const FAssetData& InAsset) const override { return FAssetSupportResponse::NotSupported(); }	

	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const FAssetCategoryPath Categories[] = { FAssetCategoryPath(EAssetCategoryPaths::World, NSLOCTEXT("AssetDefinition", "WorldBookmark_SubMenu", "World"), ECategoryMenuType::Section) };
		return Categories;
	}

	virtual bool ShouldFindEditorForAsset() const override { return bOpenFromContentBrowser; }
	virtual EAssetCommandResult ActivateAssets(const FAssetActivateArgs& ActivateArgs) const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End

private:
	mutable bool bOpenFromContentBrowser = false;
};