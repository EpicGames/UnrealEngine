// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldBookmark/Browser/BookmarkTableRow.h"
#include "WorldBookmark/Browser/Columns.h"
#include "WorldBookmark/Browser/CommonTableRow.h"
#include "WorldBookmark/Browser/Icons.h"
#include "WorldBookmark/Browser/Settings.h"
#include "WorldBookmark/WorldBookmark.h"

#include "AssetToolsModule.h"
#include "AssetViewUtils.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"

namespace UE::WorldBookmark::Browser
{

#define LOCTEXT_NAMESPACE "WorldBookmarkBrowser"

void FWorldBookmarkTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, FWorldBookmarkTreeItemRef InItem)
{
	WeakTreeItem = InItem;
	SMultiColumnTableRow<FWorldBookmarkTreeItemPtr>::Construct(
		STableRow::FArguments(InArgs)
		.OnDragDetected(this, &FWorldBookmarkTableRowBase::OnRowDragDetected),
		OwnerTable);
}

TSharedRef<SWidget> FWorldBookmarkTableRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (UWorldBookmark* WorldBookmark = GetBookmark())
	{
		if (ColumnName == Columns::Favorite.Id)
		{
			if (WorldBookmark->GetIsUserFavorite())
			{
				return SNew(SBox)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
							.DesiredSizeOverride(FVector2D(16, 16))
							.ColorAndOpacity(this, &FWorldBookmarkTableRow::GetIconColor)
							.Image(GetFavoriteWorldBookmarkIcon(true).GetIcon())
					];
			}
		}
		else if (ColumnName == Columns::RecentlyUsed.Id)
		{
			FTimespan LastLoaded = FDateTime::UtcNow() - WorldBookmark->GetUserLastLoadedTimeStampUTC();
			if (LastLoaded < FTimespan::FromDays(14))
			{
				return CreateIconWidget(GetRecentlyUsedWorldBookmarkIcon(), GetLastAccessedText());
			}
		}
		else if (ColumnName == Columns::Label.Id)
		{
			if (UWorldBookmarkBrowserSettings::IsViewMode(EWorldBookmarkBrowserViewMode::TreeView))
			{
				return GenerateLabelForTreeView();
			}
			else
			{
				return GenerateLabelForListView();
			}
		}
		else if (ColumnName == Columns::World.Id)
		{
			return SNew(SBox)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(this, &FWorldBookmarkTableRow::GetWorldNameText)
				];
		}
		else if (ColumnName == Columns::Category.Id)
		{
			return SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SColorBlock)
						.Color(this, &FWorldBookmarkTableRow::GetCategoryColor)
						.ToolTipText(this, &FWorldBookmarkTableRow::GetCategoryText)
						.CornerRadius(FVector4(4.0f, 4.0f, 4.0f, 4.0f))
				];
		}
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> FWorldBookmarkTableRow::GenerateLabelForTreeView()
{
	return CreateTreeLabelWidget(SharedThis(this));
}

TSharedRef<SWidget> FWorldBookmarkTableRow::GenerateLabelForListView()
{
	return SNew(SBox)
		.VAlign(EVerticalAlignment::VAlign_Center)
		.HeightOverride(35)
		[
			SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Padding(10, 0, 0, 0)
				.AutoHeight()
				.VAlign(EVerticalAlignment::VAlign_Center)
				[
					CreateEditableLabelWidget(SharedThis(this))
				]
				+ SVerticalBox::Slot()
				.Padding(10, 0, 0, 0)
				[
					SNew(STextBlock)
						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
						.Text(this, &FWorldBookmarkTableRow::GetLastAccessedText)
						.Visibility(this, &FWorldBookmarkTableRow::GetLastAccessedTextVisibility)
				]
		];
}

UWorldBookmark* FWorldBookmarkTableRow::GetBookmark() const
{
	if (FWorldBookmarkTreeItemPtr WorldBookmarkTreeItem = GetTreeItem())
	{
		return Cast<UWorldBookmark>(WorldBookmarkTreeItem->BookmarkAsset.GetAsset());
	}

	return nullptr;
}

FText FWorldBookmarkTableRow::GetLastAccessedText() const
{
	if (UWorldBookmark* WorldBookmark = GetBookmark())
	{
		FDateTime LastLoadedUTC = WorldBookmark->GetUserLastLoadedTimeStampUTC();
		if (LastLoadedUTC != FDateTime::MinValue())
		{
			FTimespan UTCOffset = FDateTime::UtcNow() - FDateTime::Now();
			uint64 AdjustedTicks = LastLoadedUTC.GetTicks() - UTCOffset.GetTicks();
			FDateTime LastLoadedLocal = FDateTime(AdjustedTicks);

			return FText::Format(LOCTEXT("LastAccessedDateTime", "Last accessed {0}"), FText::AsDateTime(LastLoadedUTC, TEXT("%Y-%m-%d %H:%M")));
		}
	}
	
	return FText();
}

EVisibility FWorldBookmarkTableRow::GetLastAccessedTextVisibility() const
{
	if (UWorldBookmark* WorldBookmark = GetBookmark())
	{
		FDateTime LastLoadedUTC = WorldBookmark->GetUserLastLoadedTimeStampUTC();
		if (LastLoadedUTC != FDateTime::MinValue())
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

FText FWorldBookmarkTableRow::GetWorldNameText() const
{
	if (FWorldBookmarkTreeItemPtr WorldBookmarkTreeItem = GetTreeItem())
	{
		return FText::FromString(WorldBookmarkTreeItem->GetBookmarkWorld().GetAssetName());
	}

	return FText();
}

FLinearColor FWorldBookmarkTableRow::GetCategoryColor() const
{
	if (UWorldBookmark* WorldBookmark = GetBookmark())
	{
		if (WorldBookmark->GetBookmarkCategory().Name != NAME_None)
		{
			return WorldBookmark->GetBookmarkCategory().Color;
		}
	}

	return FLinearColor::Transparent;
}

FText FWorldBookmarkTableRow::GetCategoryText() const
{
	if (UWorldBookmark* WorldBookmark = GetBookmark())
	{
		return FText::FromName(WorldBookmark->GetBookmarkCategory().Name);
	}

	return FText();
}

FSlateColor FWorldBookmarkTableRow::GetIconColor() const
{
	FLinearColor IconColor = GetCategoryColor();
	
	return IconColor == FLinearColor::Transparent ? FSlateColor::UseForeground() : IconColor;
}

#undef LOCTEXT_NAMESPACE

}