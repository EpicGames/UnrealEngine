// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"

#include "WorldBookmark/Browser/BookmarkTreeItem.h"
#include "WorldBookmark/Browser/CommonTableRow.h"

class UWorldBookmark;

namespace UE::WorldBookmark::Browser
{

class FWorldBookmarkTableRow : public SMultiColumnTableRow<FWorldBookmarkTreeItemPtr>, public FWorldBookmarkTableRowBase<FWorldBookmarkTreeItem>
{
public:
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, FWorldBookmarkTreeItemRef InItem);

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

	TSharedRef<SWidget> GenerateLabelForTreeView();
	TSharedRef<SWidget> GenerateLabelForListView();

	FText GetLastAccessedText() const;
	EVisibility GetLastAccessedTextVisibility() const;

	FText GetWorldNameText() const;

	FLinearColor GetCategoryColor() const;
	FText GetCategoryText() const;

	FSlateColor GetIconColor() const;

	UWorldBookmark* GetBookmark() const;
};

}