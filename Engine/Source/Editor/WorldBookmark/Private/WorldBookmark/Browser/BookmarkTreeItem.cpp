// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldBookmark/Browser/BookmarkTreeItem.h"
#include "WorldBookmark/WorldBookmark.h"

#include "AssetToolsModule.h"
#include "AssetViewUtils.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "ObjectTools.h"

namespace UE::WorldBookmark::Browser
{

const ETreeItemType FWorldBookmarkTreeItem::Type(ETreeItemType::WorldBookmark);

FName FWorldBookmarkTreeItem::GetName() const
{
	return BookmarkAsset.AssetName;
}

FName FWorldBookmarkTreeItem::GetAssetName() const
{
	return GetName();
}

static FSoftObjectPath GetRenamedObjectPath(FName PackagePath, FName InNewPackageName)
{
	const FString NewObjectPath = FString::Format(TEXT("{0}/{1}.{1}"), { *PackagePath.ToString(), *InNewPackageName.ToString() });
	return FSoftObjectPath(NewObjectPath);
}

bool FWorldBookmarkTreeItem::CanRename() const
{
	return true;
}

bool FWorldBookmarkTreeItem::TryRename(FName InNewName, FText& OutErrorMessage) const
{
	const FSoftObjectPath NewObjectPath = GetRenamedObjectPath(BookmarkAsset.PackagePath, InNewName);
	return AssetViewUtils::IsValidObjectPathForCreate(NewObjectPath.ToString(), OutErrorMessage);
}

bool FWorldBookmarkTreeItem::Rename(FName InNewName)
{
	FText ErrorMessage;
	if (TryRename(InNewName, ErrorMessage))
	{
		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");

		TArray<FAssetRenameData> AssetsAndNames;
		const FSoftObjectPath OldObjectPath = BookmarkAsset.GetSoftObjectPath();
		const FSoftObjectPath NewObjectPath = GetRenamedObjectPath(BookmarkAsset.PackagePath, InNewName);
		new (AssetsAndNames) FAssetRenameData(OldObjectPath, NewObjectPath);

		return AssetToolsModule.Get().RenameAssetsWithDialog(AssetsAndNames) == EAssetRenameResult::Success;
	}

	return false;
}

bool FWorldBookmarkTreeItem::Delete()
{
	return ObjectTools::DeleteAssets({ BookmarkAsset }) == 1;
}

FName FWorldBookmarkTreeItem::GetIconName() const
{
	return "ClassIcon.WorldBookmark";
}

FText FWorldBookmarkTreeItem::GetText() const
{
	return FText::FromName(BookmarkAsset.AssetName);
}

FText FWorldBookmarkTreeItem::GetTooltipText() const
{
	return FText::FromString(GetAssetPath());
}

void FWorldBookmarkTreeItem::ShowInContentBrowser() const
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	ContentBrowserModule.Get().SyncBrowserToAssets(TArray<FAssetData>({ BookmarkAsset }));
}

}
