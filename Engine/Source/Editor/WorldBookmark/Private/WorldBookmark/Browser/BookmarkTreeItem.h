// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "WorldBookmark/Browser/TreeItem.h"

#include "AssetRegistry/AssetData.h"

namespace UE::WorldBookmark::Browser
{

struct FWorldBookmarkTreeItem : public ITreeItem
{
public:
	FWorldBookmarkTreeItem(const FAssetData& InBookmarkAsset)
		: ITreeItem(Type)
		, BookmarkAsset(InBookmarkAsset)
	{
	}

	/** Static type identifier for this tree item class */
	static const ETreeItemType Type;

	FAssetData BookmarkAsset;

	virtual FName GetName() const override;
	virtual FName GetAssetName() const override;
	virtual bool CanRename() const override;
	virtual bool TryRename(FName InNewName, FText& OutErrorMessage) const override;
	virtual bool Rename(FName InNewName) override;
	virtual bool Delete() override;
	virtual FName GetIconName() const override;
	virtual FText GetText() const override;
	virtual FText GetTooltipText() const override;
	virtual void ShowInContentBrowser() const override;

	const FSoftObjectPath& GetBookmarkWorld() const;

private:
	mutable FSoftObjectPath CachedBookmarkWorld;
};

typedef TSharedPtr<FWorldBookmarkTreeItem> FWorldBookmarkTreeItemPtr;
typedef TSharedRef<FWorldBookmarkTreeItem> FWorldBookmarkTreeItemRef;

}
