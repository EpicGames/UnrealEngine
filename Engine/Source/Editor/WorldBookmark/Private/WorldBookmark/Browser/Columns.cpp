// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldBookmark/Browser/Columns.h"

#include "Internationalization/Internationalization.h"

namespace UE::WorldBookmark::Browser
{

#define LOCTEXT_NAMESPACE "WorldBookmarkBrowser"

namespace Columns
{
	const FColumnDefinition Favorite =
	{
		TEXT("Favorite"),
		LOCTEXT("Column_Favorite_Text", "Favorite"),
		LOCTEXT("Column_Favorite_Tooltip", "Favorite")
	};

	const FColumnDefinition RecentlyUsed =
	{
		TEXT("RecentlyUsed"),
		LOCTEXT("Column_RecentlyUsed_Text", "Recently Used"),
		LOCTEXT("Column_RecentlyUsed_Tooltip", "Recently Used")
	};

	const FColumnDefinition Label =
	{
		TEXT("Label"),
		LOCTEXT("Column_Label_Text", "Name"),
		LOCTEXT("Column_Label_Tooltip", "World Bookmark Name")
	};

	const FColumnDefinition World =
	{
		TEXT("World"),
		LOCTEXT("Column_World_Text", "World"),
		LOCTEXT("Column_World_Tooltip", "World Name")
	};

	const FColumnDefinition Category =
	{
		TEXT("Category"),
		LOCTEXT("Column_Category_Text", "Category"),
		LOCTEXT("Column_Category_Tooltip", "Category")
	};

	const TArray<FName> DefaultHiddenColumns = { World.Id };
}

#undef LOCTEXT_NAMESPACE

}
