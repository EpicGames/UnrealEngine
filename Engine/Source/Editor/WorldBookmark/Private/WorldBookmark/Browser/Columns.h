// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Internationalization/Text.h"
#include "UObject/NameTypes.h"

namespace UE::WorldBookmark::Browser::Columns
{
	struct FColumnDefinition
	{
		FName Id;
		FText DisplayText;
		FText ToolTipText;
	};

	extern const FColumnDefinition Favorite;
	extern const FColumnDefinition RecentlyUsed;
	extern const FColumnDefinition Label;
	extern const FColumnDefinition World;
	extern const FColumnDefinition Category;

	extern const TArray<FName> DefaultHiddenColumns;
}
