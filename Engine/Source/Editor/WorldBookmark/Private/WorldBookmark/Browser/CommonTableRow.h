// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Textures/SlateIcon.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Views/STableRow.h"
#include "WorldBookmark/Browser/DragDropOperation.h"
#include "WorldBookmark/Browser/TreeItem.h"
#include "WorldBookmark/WorldBookmarkStyle.h"

namespace UE::WorldBookmark::Browser
{

template <class TItemType = ITreeItem>
class FWorldBookmarkTableRowBase
{
public:	
	TSharedPtr<TItemType> GetTreeItem() const
	{
		if (FTreeItemPtr TreeItem = WeakTreeItem.Pin())
		{
			if (TreeItem->IsA<TItemType>())
			{
				return StaticCastSharedPtr<TItemType>(TreeItem);
			}
		}

		return nullptr;
	}

	FText GetText() const
	{
		if (FTreeItemPtr TreeItem = GetTreeItem())
		{
			return FText::FromName(TreeItem->GetName());
		}

		return FText();
	}

	FText GetTooltipText() const
	{
		if (FTreeItemPtr TreeItem = GetTreeItem())
		{
			return FText::FromString(TreeItem->GetAssetPath());
		}

		return FText();
	}

	const FSlateBrush* GetIcon() const
	{
		if (FTreeItemPtr TreeItem = GetTreeItem())
		{
			return FSlateIcon(FWorldBookmarkStyle::Get().GetStyleSetName(), TreeItem->GetIconName()).GetIcon();
		}

		return nullptr;
	}

	FSlateColor GetIconColor() const
	{
		return FSlateColor::UseForeground();
	}

	FSimpleDelegate& GetRenameRequestedDelegate()
	{
		return WeakTreeItem.Pin()->OnRenameRequested;
	}

	bool IsReadOnly() const
	{
		if (FTreeItemPtr TreeItem = GetTreeItem())
		{
			return !TreeItem->CanRename();
		}

		return true;
	}

	bool OnLabelTextVerifyChanged(const FText& InNewText, FText& OutErrorMessage)
	{
		if (FTreeItemPtr TreeItem = GetTreeItem())
		{
			FName OldName(TreeItem->GetName());
			FName NewName(InNewText.ToString());

			return OldName == NewName || TreeItem->TryRename(FName(InNewText.ToString()), OutErrorMessage);
		}

		return false;
	}

	void OnLabelTextCommitted(const FText& InNewText, ETextCommit::Type InTextCommit)
	{
		if (FTreeItemPtr TreeItem = GetTreeItem())
		{
			FName OldName(TreeItem->GetName());
			FName NewName(InNewText.ToString());

			if (OldName != NewName)
			{
				TreeItem->Rename(FName(InNewText.ToString()));
			}
		}
	}

	void OnEnterEditingMode()
	{
		if (FTreeItemPtr TreeItem = GetTreeItem())
		{
			TreeItem->bInEditingMode = true;
		}
	}

	void OnExitEditingMode()
	{
		if (FTreeItemPtr TreeItem = GetTreeItem())
		{
			TreeItem->bInEditingMode = false;
		}
	}

	FReply OnRowDragDetected(const FGeometry& Geometry, const FPointerEvent& PointerEvent)
	{
		if (FTreeItemPtr TreeItem = GetTreeItem())
		{
			const TSharedPtr<FTableRowDragDropOp> DragDropOp = FTableRowDragDropOp::New(TreeItem);
			if (DragDropOp.IsValid())
			{
				return FReply::Handled().BeginDragDrop(DragDropOp.ToSharedRef());
			}
		}

		return FReply::Unhandled();
	}

	TOptional<EItemDropZone> OnRowCanAcceptDrop(const FDragDropEvent& Event, EItemDropZone Zone, TSharedPtr<TItemType> Item)
	{
		const TSharedPtr<FTableRowDragDropOp> DragDropOp = Event.GetOperationAs<FTableRowDragDropOp>();
		if (DragDropOp.IsValid())
		{
			return DragDropOp->CanAcceptDrop(Item);
		}

		return TOptional<EItemDropZone>();
	}

	FReply OnRowAcceptDrop(const FDragDropEvent& Event, EItemDropZone Zone, TSharedPtr<TItemType> Item)
	{
		const TSharedPtr<FTableRowDragDropOp> DragDropOp = Event.GetOperationAs<FTableRowDragDropOp>();
		if (DragDropOp.IsValid())
		{
			return DragDropOp->AcceptDrop(Item);
		}

		return FReply::Unhandled();
	}
		
protected:
	TWeakPtr<TItemType> WeakTreeItem;
};

template <class TTableRow>
TSharedRef<SWidget> CreateEditableLabelWidget(TSharedRef<TTableRow> TableRow)
{
	TSharedRef<SInlineEditableTextBlock> InlineEditableTextBlock = SNew(SInlineEditableTextBlock)
		.Text(TableRow->GetText())
		.ToolTipText(TableRow->GetTooltipText())
		.OnVerifyTextChanged(TableRow, &TTableRow::OnLabelTextVerifyChanged)
		.OnTextCommitted(TableRow, &TTableRow::OnLabelTextCommitted)
		.OnEnterEditingMode(TableRow, &TTableRow::OnEnterEditingMode)
		.OnExitEditingMode(TableRow, &TTableRow::OnExitEditingMode)
		.IsReadOnly(TableRow, &TTableRow::IsReadOnly)
		.IsSelected(TableRow, &TTableRow::IsSelected);

	TableRow->GetRenameRequestedDelegate().BindSP(InlineEditableTextBlock, &SInlineEditableTextBlock::EnterEditingMode);

	return InlineEditableTextBlock;
}

template <class TTableRow>
TSharedRef<SWidget> CreateTreeLabelWidget(TSharedRef<TTableRow> TableRow)
{
	static const int32 RowHeight = 20;
	static const int32 IconSize = 16;
	static const FMargin IconPadding(0.f, 1.f, 3.f, 1.f);

	return SNew(SBox)
	.MinDesiredHeight(RowHeight)
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(4, 0, 0, 0)
		[
			SNew(SExpanderArrow, TableRow).IndentAmount(12)
		]

		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(IconPadding)
			[
				SNew(SBox)
				.WidthOverride(IconSize)
				.HeightOverride(IconSize)
				[
					SNew(SImage)
					.Image_Lambda([TableRow] { return TableRow->GetIcon(); })
					.ToolTipText(TableRow->GetTooltipText())
					.ColorAndOpacity_Lambda([TableRow] { return TableRow->GetIconColor(); })
				]
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f)
			[
				CreateEditableLabelWidget(TableRow)
			]
		]
	];
}

}
