// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldBookmark/Browser/DragDropOperation.h"

#include "Textures/SlateIcon.h"
#include "WorldBookmark/Browser/FolderTreeItem.h"
#include "WorldBookmark/Browser/Settings.h"
#include "WorldBookmark/WorldBookmarkStyle.h"

namespace UE::WorldBookmark::Browser
{

TSharedPtr<FTableRowDragDropOp> FTableRowDragDropOp::New(FTreeItemPtr InTreeItem)
{
	if (UWorldBookmarkBrowserSettings::IsViewMode(EWorldBookmarkBrowserViewMode::TreeView))
	{
		if (InTreeItem.IsValid())
		{
			if (InTreeItem->CanRename())
			{
				TSharedRef<FTableRowDragDropOp> Operation = MakeShared<FTableRowDragDropOp>();

				Operation->CurrentHoverText = InTreeItem->GetText();
				Operation->CurrentIconBrush = FSlateIcon(FWorldBookmarkStyle::Get().GetStyleSetName(), InTreeItem->GetIconName()).GetIcon();
				Operation->SetupDefaults();

				Operation->DraggedItem = InTreeItem;
				Operation->Construct();

				return Operation.ToSharedPtr();
			}
		}
	}

	return nullptr;
}

TOptional<EItemDropZone> FTableRowDragDropOp::CanAcceptDrop(FTreeItemPtr DropTarget) const
{
	if (DropTarget.IsValid())
	{
		if (FFolderTreeItem* FolderTreeItem = DropTarget->Cast<FFolderTreeItem>())
		{
			if (!FolderTreeItem->IsVirtual())
			{
				return EItemDropZone::OntoItem;
			}
		}
	}

	return TOptional<EItemDropZone>();
}

FReply FTableRowDragDropOp::AcceptDrop(FTreeItemPtr DropTarget) const
{
	if (ensure(CanAcceptDrop(DropTarget) == EItemDropZone::OntoItem))
	{
		if (FFolderTreeItem* FolderTreeItem = DropTarget->Cast<FFolderTreeItem>())
		{
			FolderTreeItem->Move(DraggedItem.Pin());
		}
	}

	return FReply::Handled().EndDragDrop();
}

}
