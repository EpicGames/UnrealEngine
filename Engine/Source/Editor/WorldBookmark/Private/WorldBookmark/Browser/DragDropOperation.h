// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Widgets/Views/STableRow.h"
#include "WorldBookmark/Browser/TreeItem.h"

namespace UE::WorldBookmark::Browser
{

class FTableRowDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FTableRowDragDropOp, FDecoratedDragDropOp)
	static TSharedPtr<FTableRowDragDropOp> New(FTreeItemPtr TreeItem);

	TOptional<EItemDropZone> CanAcceptDrop(FTreeItemPtr DropTarget) const;
	FReply AcceptDrop(FTreeItemPtr DropTarget) const;

	TWeakPtr<ITreeItem> DraggedItem;
};

}
