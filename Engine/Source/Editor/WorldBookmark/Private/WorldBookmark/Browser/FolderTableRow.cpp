// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldBookmark/Browser/FolderTableRow.h"
#include "WorldBookmark/Browser/Icons.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"

namespace UE::WorldBookmark::Browser
{

void FFolderTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, FFolderTreeItemRef InItem)
{
	WeakTreeItem = InItem;
	STableRow::Construct(
		STableRow::FArguments(InArgs)
		.OnDragDetected(this, &FWorldBookmarkTableRowBase::OnRowDragDetected)
		.OnCanAcceptDrop(this, &FWorldBookmarkTableRowBase::OnRowCanAcceptDrop)
		.OnAcceptDrop(this, &FWorldBookmarkTableRowBase::OnRowAcceptDrop),
		InOwnerTableView);

	ChildSlot
	[
		CreateTreeLabelWidget(SharedThis(this))
	];
}

}
