// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"

#include "WorldBookmark/Browser/FolderTreeItem.h"
#include "WorldBookmark/Browser/CommonTableRow.h"

namespace UE::WorldBookmark::Browser
{

class FFolderTableRow : public STableRow<FFolderTreeItemPtr>, public FWorldBookmarkTableRowBase<FFolderTreeItem>
{
public:
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, FFolderTreeItemRef InItem);
};

}
