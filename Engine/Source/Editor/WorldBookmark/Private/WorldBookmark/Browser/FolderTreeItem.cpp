// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldBookmark/Browser/FolderTreeItem.h"

#include "WorldBookmark/Browser/BookmarkTreeItem.h"

#include "AssetToolsModule.h"
#include "AssetViewUtils.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "ObjectTools.h"

namespace UE::WorldBookmark::Browser
{

const ETreeItemType FFolderTreeItem::Type(ETreeItemType::Folder);


FFolderTreeItem::FFolderTreeItem(const FName InName)
	: ITreeItem(Type)
	, Name(InName)
	, bIsExpanded(true)
	, bIsVirtual(false)
	, bIsMountPoint(false)
{
}

FName FFolderTreeItem::GetName() const
{
	return Name;
}

FName FFolderTreeItem::GetAssetName() const
{
	if (bIsVirtual)
	{
		return NAME_None;
	}
	else if (bIsMountPoint)
	{
		return MountPointName;
	}
	else
	{
		return Name;
	}
}

bool FFolderTreeItem::CanRename() const
{
	return !IsVirtual() && !IsMountPoint();
}

bool FFolderTreeItem::TryRename(FName InNewName, FText& OutErrorMessage) const
{
	return AssetViewUtils::IsValidFolderPathForCreate(GetAssetPath(), InNewName.ToString(), OutErrorMessage);
}

bool FFolderTreeItem::Rename(FName InNewName)
{
	check(!bIsVirtual && !bIsMountPoint);

	FString OldPathToFolder = GetAssetPath();
	FString NewPathToFolder = (GetParent() ? GetParent()->GetAssetPath() : FString()) / InNewName.ToString();

	TArray<FAssetRenameData> AssetsAndNames;

	ForEachChildRecursive<FWorldBookmarkTreeItem>([&AssetsAndNames, &OldPathToFolder, &NewPathToFolder](FWorldBookmarkTreeItemPtr WorldBookmarkItem)
	{
		FSoftObjectPath OldObjectPath = WorldBookmarkItem->BookmarkAsset.GetSoftObjectPath();
		FSoftObjectPath NewObjectPath = OldObjectPath.ToString().Replace(*OldPathToFolder, *NewPathToFolder);
		new (AssetsAndNames) FAssetRenameData(OldObjectPath, NewObjectPath);
	});	

	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	return AssetToolsModule.Get().RenameAssetsWithDialog(AssetsAndNames) == EAssetRenameResult::Success;
}

bool FFolderTreeItem::Move(FTreeItemPtr NewChild)
{
	TArray<FAssetRenameData> AssetsAndNames;

	FString OldPathToItem = NewChild->GetAssetPath();
	FString NewPathToItem = GetAssetPath() / NewChild->GetName().ToString();

	auto AddAssetToRename = [&AssetsAndNames, &OldPathToItem, &NewPathToItem](FWorldBookmarkTreeItem* WorldBookmarkItem)
	{
		FSoftObjectPath OldObjectPath = WorldBookmarkItem->BookmarkAsset.GetSoftObjectPath();
		FSoftObjectPath NewObjectPath = OldObjectPath.ToString().Replace(*OldPathToItem, *NewPathToItem);
		new (AssetsAndNames) FAssetRenameData(OldObjectPath, NewObjectPath);
	};

	if (FWorldBookmarkTreeItem* WorldBookmarkItem = NewChild->Cast<FWorldBookmarkTreeItem>())
	{
		AddAssetToRename(WorldBookmarkItem);
	}
	else if (FFolderTreeItem* FolderTreeItem = NewChild->Cast<FFolderTreeItem>())
	{
		FolderTreeItem->ForEachChildRecursive<FWorldBookmarkTreeItem>([&AddAssetToRename](FWorldBookmarkTreeItemPtr WorldBookmarkItem)
		{
			AddAssetToRename(WorldBookmarkItem.Get());
		});
	}

	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	return AssetToolsModule.Get().RenameAssetsWithDialog(AssetsAndNames) == EAssetRenameResult::Success;
}

bool FFolderTreeItem::Delete()
{
	TArray<FAssetData> AssetsToDelete;

	ForEachChildRecursive<FWorldBookmarkTreeItem>([&AssetsToDelete](FWorldBookmarkTreeItemPtr WorldBookmarkItem)
	{
		AssetsToDelete.Add(WorldBookmarkItem->BookmarkAsset);
	});

	return ObjectTools::DeleteAssets({ AssetsToDelete }) == AssetsToDelete.Num();
}

FName FFolderTreeItem::GetIconName() const
{
	return bIsExpanded ? "WorldBookmark.FolderOpen" : "WorldBookmark.FolderClosed";
}

FText FFolderTreeItem::GetText() const
{
	return FText::FromName(Name);
}

FText FFolderTreeItem::GetTooltipText() const
{
	return FText::FromString(GetAssetPath());
}

void FFolderTreeItem::ShowInContentBrowser() const
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	ContentBrowserModule.Get().SyncBrowserToFolders({ GetAssetPath() });
}

const TArray<FTreeItemPtr>& FFolderTreeItem::GetChildren() const
{
	return Children;
}

void FFolderTreeItem::AddChild(FTreeItemPtr InChildItem)
{
	Children.Add(InChildItem);
	InChildItem->SetParent(this);
}

FFolderTreeItemPtr FFolderTreeItem::CreateMountPoint(const FString& InPath, const FString& InMountPointName)
{
	FFolderTreeItemPtr MountPoint = CreatePath(InPath, true);
	MountPoint->bIsMountPoint = true;
	MountPoint->MountPointName = FName(InMountPointName);
	return MountPoint;
}

FFolderTreeItemPtr FFolderTreeItem::CreatePath(const FString& InPath)
{
	return CreatePath(InPath, false);
}

FFolderTreeItemPtr FFolderTreeItem::CreatePath(const FString& InPath, bool bInIsMountPoint)
{
	TArray<FString> PathAsStringArray;
	InPath.ParseIntoArray(PathAsStringArray, TEXT("/"), true);

	TArray<FName> PathAsNameArray;
	Algo::Transform(PathAsStringArray, PathAsNameArray, [](const FString& InString) { return FName(InString); });
	return CreatePath(PathAsNameArray, bInIsMountPoint);
}

FFolderTreeItemPtr FFolderTreeItem::CreatePath(const TArrayView<FName>& InPath, bool bInIsCreatingMountPoint)
{
	if (InPath.IsEmpty())
	{
		return StaticCastSharedPtr<FFolderTreeItem>(AsShared().ToSharedPtr());
	}

	FFolderTreeItemPtr NextFolder = nullptr;
	for (TSharedPtr<ITreeItem>& Child : Children)
	{
		if (Child->IsA<FFolderTreeItem>() && Child->GetName() == InPath[0])
		{
			NextFolder = StaticCastSharedPtr<FFolderTreeItem>(Child);
			break;
		}
	}

	if (NextFolder == nullptr)
	{
		NextFolder = MakeShared<FFolderTreeItem>(InPath[0]);
		NextFolder->SetParent(this);
		Children.Add(NextFolder);
	}

	if (InPath.Num() == 1)
	{
		// We're done, that was the last folder in the path
		return NextFolder;
	}
	else
	{
		// Folders above a mount point does not exists, mark them as "virtual"
		NextFolder->bIsVirtual = bInIsCreatingMountPoint;

		// Recurse deeper in the tree
		return NextFolder->CreatePath(InPath.RightChop(1), bInIsCreatingMountPoint);
	}
}

void FFolderTreeItem::ExpandPath(const FName InPath) const
{
	TArray<FString> PathAsStringArray;
	InPath.ToString().ParseIntoArray(PathAsStringArray, TEXT("/"), true);
	
	TArray<FName> PathAsNameArray;
	Algo::Transform(PathAsStringArray, PathAsNameArray, [](const FString& InString) { return FName(InString); });
	return ExpandPath(PathAsNameArray);
}

void FFolderTreeItem::ExpandPath(const TArrayView<FName>& InPath) const
{

}

bool FFolderTreeItem::IsExpanded() const
{
	return bIsExpanded;
}

void FFolderTreeItem::SetExpanded(bool bInExpanded)
{
	bIsExpanded = bInExpanded;
}

bool FFolderTreeItem::IsVirtual() const
{
	return bIsVirtual;
}

bool FFolderTreeItem::IsMountPoint() const
{
	return bIsMountPoint;
}

void FFolderTreeItem::ClearBookmarkItems()
{
	for (TArray<FTreeItemPtr>::TIterator It = Children; It; ++It)
	{
		TSharedPtr<ITreeItem>& Child = *It;
		if (FWorldBookmarkTreeItem* BookmarkTreeItem = Child->Cast<FWorldBookmarkTreeItem>())
		{
			It.RemoveCurrent();
		}
		else if (FFolderTreeItem* ChildFolder = Child->Cast<FFolderTreeItem>())
		{
			ChildFolder->ClearBookmarkItems();
		}
	}
}

bool FFolderTreeItem::ClearEmptyFolders()
{
	for (TArray<FTreeItemPtr>::TIterator It = Children; It; ++It)
	{
		TSharedPtr<ITreeItem>& Child = *It;
		if (FFolderTreeItem* ChildFolder = Child->Cast<FFolderTreeItem>())
		{
			bool bIsEmpty = ChildFolder->ClearEmptyFolders();
			if (bIsEmpty)
			{
				It.RemoveCurrent();
			}
		}
	}

	return Children.IsEmpty();
}

void FFolderTreeItem::Sort(TFunctionRef<bool(const FTreeItemPtr& A, const FTreeItemPtr& B)> SortFunc)
{
	Children.Sort(SortFunc);

	for (TSharedPtr<ITreeItem>& Child : Children)
	{
		if (FFolderTreeItem* ChildFolder = Child->Cast<FFolderTreeItem>())
		{
			ChildFolder->Sort(SortFunc);
		}
	}
}

void FFolderTreeItem::Reset()
{
	Children.Reset();
}

}
