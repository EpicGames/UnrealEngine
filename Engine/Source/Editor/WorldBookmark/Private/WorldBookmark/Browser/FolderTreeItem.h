// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "WorldBookmark/Browser/TreeItem.h"

namespace UE::WorldBookmark::Browser
{

struct FFolderTreeItem : public ITreeItem
{
public:
	/** Static type identifier for this tree item class */
	static const ETreeItemType Type;

	FFolderTreeItem(const FName InName = NAME_None);

	virtual FName GetName() const override;
	virtual FName GetAssetName() const override;
	virtual bool CanRename() const override;
	virtual bool TryRename(FName InNewName, FText& OutErrorMessage) const override;
	virtual bool Rename(FName InNewName) override;
	virtual bool Delete() override;
	virtual FName GetIconName() const override;
	virtual FText GetText() const override;
	virtual FText GetTooltipText() const override;
	virtual void ShowInContentBrowser() const override;

	const TArray<FTreeItemPtr>& GetChildren() const;
	void AddChild(FTreeItemPtr InChildItem);

	TSharedPtr<FFolderTreeItem> CreateMountPoint(const FString& InPath, const FString& InMountPointName);
	TSharedPtr<FFolderTreeItem> CreatePath(const FString& InPath);
	void ExpandPath(FName InPath) const;

	bool IsExpanded() const;
	void SetExpanded(bool bInExpanded);

	bool IsVirtual() const;
	bool IsMountPoint() const;

	void Sort(TFunctionRef<bool(const FTreeItemPtr&, const FTreeItemPtr&)> SortFunc);

	void Reset();

	void ClearBookmarkItems();
	bool ClearEmptyFolders();

	bool Move(FTreeItemPtr NewChild);

	template <typename T>
	void ForEachChild(TFunctionRef<void(TSharedPtr<T>)> InFunc)
	{
		for (FTreeItemPtr& Child : Children)
		{
			if (Child->IsA<T>())
			{
				InFunc(StaticCastSharedPtr<T>(Child));
			}
		}
	}

	template <typename T>
	void ForEachChildRecursive(TFunctionRef<void(TSharedPtr<T>)> InFunc)
	{
		for (FTreeItemPtr& Child : Children)
		{
			if (Child->IsA<T>())
			{
				InFunc(StaticCastSharedPtr<T>(Child));
			}

			if (FFolderTreeItem* Folder = Child->Cast<FFolderTreeItem>())
			{
				Folder->ForEachChildRecursive(InFunc);
			}
		}
	}

private:
	TSharedPtr<FFolderTreeItem> CreatePath(const FString& InPath, bool bIsCreatingMountPoint);
	TSharedPtr<FFolderTreeItem> CreatePath(const TArrayView<FName>& InPath, bool bIsCreatingMountPoint);
	void ExpandPath(const TArrayView<FName>& InPath) const;

	FName Name;
	FName MountPointName;
	bool bIsExpanded;
	bool bIsVirtual;
	bool bIsMountPoint;
	TArray<FTreeItemPtr> Children;
};

typedef TSharedPtr<FFolderTreeItem> FFolderTreeItemPtr;
typedef TSharedRef<FFolderTreeItem> FFolderTreeItemRef;

}
