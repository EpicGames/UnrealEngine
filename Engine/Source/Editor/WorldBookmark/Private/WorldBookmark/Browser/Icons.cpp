// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldBookmark/Browser/Icons.h"

#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "WorldBookmark/WorldBookmarkStyle.h"

namespace UE::WorldBookmark::Browser
{

FSlateIcon GetFavoriteWorldBookmarkIcon(bool bIsFavorite)
{
	const FName FavoriteIconName = bIsFavorite ? "WorldBookmark.IsFavorite" : "WorldBookmark.IsNotFavorite";
	return FSlateIcon(FWorldBookmarkStyle::Get().GetStyleSetName(), FavoriteIconName);
}

FSlateIcon GetRecentlyUsedWorldBookmarkIcon()
{
	const FName RecentlyUsedIconName = "WorldBookmark.RecentlyUsed";
	return FSlateIcon(FWorldBookmarkStyle::Get().GetStyleSetName(), RecentlyUsedIconName);
}

FSlateIcon GetFolderIcon(bool bIsExpanded)
{
	const FName FolderIconName = bIsExpanded ? "WorldBookmark.FolderOpen" : "WorldBookmark.FolderClosed";
	return FSlateIcon(FWorldBookmarkStyle::Get().GetStyleSetName(), FolderIconName);
}

FSlateIcon GetWorldBookmarkIcon()
{
	const FName WorldBookmarkIconName = "ClassIcon.WorldBookmark";
	return FSlateIcon(FWorldBookmarkStyle::Get().GetStyleSetName(), WorldBookmarkIconName);
}

TSharedRef<SWidget> CreateIconWidget(const FSlateIcon& SlateIcon, const FText& ToolTip)
{
	return SNew(SBox)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
				.DesiredSizeOverride(FVector2D(16, 16))
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(SlateIcon.GetIcon())
				.ToolTipText(ToolTip)
		];
}

}
