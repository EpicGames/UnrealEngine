// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"

class SWidget;

namespace UE::WorldBookmark::Browser
{
	FSlateIcon GetFavoriteWorldBookmarkIcon(bool bIsFavorite);
	FSlateIcon GetRecentlyUsedWorldBookmarkIcon();
	FSlateIcon GetFolderIcon(bool bIsExpanded);
	FSlateIcon GetWorldBookmarkIcon();
	TSharedRef<SWidget> CreateIconWidget(const FSlateIcon& SlateIcon, const FText& ToolTip = FText());
}