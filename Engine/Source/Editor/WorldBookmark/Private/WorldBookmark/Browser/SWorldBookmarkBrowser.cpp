// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldBookmark/Browser/SWorldBookmarkBrowser.h"

#include "WorldBookmark/Browser/Columns.h"
#include "WorldBookmark/Browser/Icons.h"
#include "WorldBookmark/Browser/Settings.h"
#include "WorldBookmark/Browser/BookmarkTreeItem.h"
#include "WorldBookmark/Browser/BookmarkTableRow.h"
#include "WorldBookmark/Browser/FolderTreeItem.h"
#include "WorldBookmark/Browser/FolderTableRow.h"

#include "WorldBookmark/WorldBookmark.h"
#include "WorldBookmark/WorldBookmarkCommands.h"
#include "WorldBookmark/WorldBookmarkFactory.h"
#include "WorldBookmark/WorldBookmarkStyle.h"

#include "Algo/AnyOf.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryHelpers.h"
#include "AssetToolsModule.h"
#include "ContentBrowserModule.h"
#include "Dialogs/Dialogs.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "IContentBrowserDataModule.h"
#include "Interfaces/IPluginManager.h"
#include "Kismet2/DebuggerCommands.h"
#include "LevelEditor.h"
#include "LevelEditorCameraEditorState.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SLevelViewport.h"
#include "SourceControlHelpers.h"
#include "SSimpleButton.h"
#include "Toolkits/GlobalEditorCommonCommands.h"
#include "UncontrolledChangelistsModule.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Views/SHeaderRow.h"

namespace UE::WorldBookmark::Browser
{

#define LOCTEXT_NAMESPACE "WorldBookmarkBrowser"

static UWorld* GetCurrentWorld()
{
	if (GEditor)
	{
		if (UWorld* EditorWorld = GEditor->GetEditorWorldContext().World())
		{
			return EditorWorld;
		}
	}
	return nullptr;
}

SWorldBookmarkBrowser::SWorldBookmarkBrowser()
	: bIsInTick(false)
	, bPendingRefresh(false)
	, bExpandAllItemsOnNextRefresh(false)
{
	// Listen for asset registry updates
	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
	OnAssetAddedHandle = AssetRegistry.OnAssetsAdded().AddRaw(this, &SWorldBookmarkBrowser::OnAssetsAdded);
	OnAssetRemovedHandle = AssetRegistry.OnAssetRemoved().AddRaw(this, &SWorldBookmarkBrowser::OnAssetRemoved);
	OnAssetRenamedHandle = AssetRegistry.OnAssetRenamed().AddRaw(this, &SWorldBookmarkBrowser::OnAssetRenamed);
	OnAssetUpdatedHandle = AssetRegistry.OnAssetUpdated().AddRaw(this, &SWorldBookmarkBrowser::OnAssetUpdated);
	OnAssetUpdatedOnDiskHandle = AssetRegistry.OnAssetUpdatedOnDisk().AddRaw(this, &SWorldBookmarkBrowser::OnAssetUpdatedOnDisk);
	
	// Listen for when assets are loaded or changed
	OnObjectPropertyChangedHandle = FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &SWorldBookmarkBrowser::OnObjectPropertyChanged);

	// Listen for map change events
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
	OnMapChangedHandle = LevelEditorModule.OnMapChanged().AddRaw(this, &SWorldBookmarkBrowser::OnMapChanged);

	ColumnsSortingParams[EColumnSortPriority::Primary] = { Columns::Favorite.Id, EColumnSortMode::Ascending };
	ColumnsSortingParams[EColumnSortPriority::Secondary] = { Columns::Label.Id, EColumnSortMode::Ascending };

	OnSettingsChangedHandle = UWorldBookmarkBrowserSettings::OnSettingsChanged().AddRaw(this, &SWorldBookmarkBrowser::OnSettingsChanged);
}

SWorldBookmarkBrowser::~SWorldBookmarkBrowser()
{
	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
	AssetRegistry.OnAssetsAdded().Remove(OnAssetAddedHandle);
	AssetRegistry.OnAssetRemoved().Remove(OnAssetRemovedHandle);
	AssetRegistry.OnAssetRenamed().Remove(OnAssetRenamedHandle);
	AssetRegistry.OnAssetUpdated().Remove(OnAssetUpdatedHandle);
	AssetRegistry.OnAssetUpdatedOnDisk().Remove(OnAssetUpdatedOnDiskHandle);
	
	FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(OnObjectPropertyChangedHandle);

	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
	LevelEditorModule.OnMapChanged().Remove(OnMapChangedHandle);

	UWorldBookmarkBrowserSettings::OnSettingsChanged().Remove(OnSettingsChangedHandle);
}

void SWorldBookmarkBrowser::Construct(const FArguments& InArgs)
{
	TreeRoot = MakeShared<FFolderTreeItem>();

	// Create a property view
	FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bHideSelectionTip = true;
	PropertyView = EditModule.CreateDetailView(DetailsViewArgs);
	PropertyView->SetObject(nullptr, true);

	HeaderRow = SNew(SHeaderRow)
		.CanSelectGeneratedColumn(true)
		.OnHiddenColumnsListChanged_Lambda([this]()
		{
			UWorldBookmarkBrowserSettings::SetHiddenColumns(BookmarksView->GetHeaderRow()->GetHiddenColumnIds());
		});

	BookmarksView = SNew(STreeView<FTreeItemPtr>)
		.TreeItemsSource(&TreeRoot->GetChildren())
		.SelectionMode(ESelectionMode::Single)
		.OnGetChildren(this, &SWorldBookmarkBrowser::OnGetChildren)
		.OnGenerateRow(this, &SWorldBookmarkBrowser::OnGenerateRow)
		.OnSelectionChanged(this, &SWorldBookmarkBrowser::OnSelectionChanged)
		.OnMouseButtonDoubleClick(this, &SWorldBookmarkBrowser::OnMouseButtonDoubleClicked)
		.OnContextMenuOpening(this, &SWorldBookmarkBrowser::OnContextMenuOpening)
		.OnItemScrolledIntoView(this, &SWorldBookmarkBrowser::OnItemScrolledIntoView)
		.OnExpansionChanged(this, &SWorldBookmarkBrowser::OnItemExpansionChanged)
		.OnSetExpansionRecursive(this, &SWorldBookmarkBrowser::SetItemExpansionRecursive)
		.HeaderRow
		(
			HeaderRow
		);

	TSharedRef<SSearchBox> SearchBoxRef = SNew(SSearchBox)
		.OnTextChanged(this, &SWorldBookmarkBrowser::OnSearchBoxTextChanged);

	ChildSlot
	[
		SNew(SSplitter)
		.Orientation(Orient_Vertical)
		.Style(FAppStyle::Get(), "DetailsView.Splitter")

		// Bookmark List
		+ SSplitter::Slot()
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(8.0f, 8.0f, 8.0f, 4.0f)
			.VAlign(VAlign_Top)
			.HAlign(HAlign_Fill)
			[
				SNew(SHorizontalBox)
				
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Fill)
				.FillWidth(1.0f)
				[
					SearchBoxRef
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.ToolTipText(LOCTEXT("CreateNewWorldBookmak", "Create a new world bookmark"))
					.OnClicked(this, &SWorldBookmarkBrowser::OnCreateNewWorldBookmarkClicked)
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FWorldBookmarkStyle::Get().GetBrush("WorldBookmark.CreateBookmark"))
					]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SComboButton)
					.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButtonWithIcon")
					.OnGetMenuContent(this, &SWorldBookmarkBrowser::GetSettingsMenuContent)
					.HasDownArrow(false)
					.ButtonContent()
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FAppStyle::Get().GetBrush("Icons.Settings"))
					]
				]
			]

			+ SVerticalBox::Slot()
			[
				BookmarksView.ToSharedRef()
			]
		]

		// Details Property View
		+ SSplitter::Slot()
		[
			SAssignNew(DetailsBox, SVerticalBox)
		]
	];

	Commands = MakeShareable(new FUICommandList());

	Commands->MapAction(FGlobalEditorCommonCommands::Get().FindInContentBrowser,
		FExecuteAction::CreateSP(this, &SWorldBookmarkBrowser::FindSelectedItemInContentBrowser),
		FCanExecuteAction::CreateSP(this, &SWorldBookmarkBrowser::IsValidItemSelected));

	Commands->MapAction(FGenericCommands::Get().Rename,
		FExecuteAction::CreateSP(this, &SWorldBookmarkBrowser::RequestRenameSelectedItem),
		FCanExecuteAction::CreateSP(this, &SWorldBookmarkBrowser::CanRenameOrDeleteSelectedItem));

	Commands->MapAction(FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &SWorldBookmarkBrowser::DeleteSelectedItem),
		FCanExecuteAction::CreateSP(this, &SWorldBookmarkBrowser::CanRenameOrDeleteSelectedItem));

	Commands->MapAction(FWorldBookmarkCommands::Get().LoadBookmark,
		FExecuteAction::CreateSP(this, &SWorldBookmarkBrowser::LoadSelectedBookmark),
		FCanExecuteAction::CreateSP(this, &SWorldBookmarkBrowser::CanExecuteBookmarkAction));

	Commands->MapAction(FWorldBookmarkCommands::Get().UpdateBookmark,
		FExecuteAction::CreateSP(this, &SWorldBookmarkBrowser::UpdateSelectedBookmark),
		FCanExecuteAction::CreateSP(this, &SWorldBookmarkBrowser::CanExecuteBookmarkAction));
	
	Commands->MapAction(FWorldBookmarkCommands::Get().AddToFavorite,
		FExecuteAction::CreateSP(this, &SWorldBookmarkBrowser::AddSelectedBookmarkToFavorites),
		FCanExecuteAction::CreateSP(this, &SWorldBookmarkBrowser::IsValidBookmarkSelected),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &SWorldBookmarkBrowser::IsSelectedBookmarkNotFavorite));

	Commands->MapAction(FWorldBookmarkCommands::Get().RemoveFromFavorite,
		FExecuteAction::CreateSP(this, &SWorldBookmarkBrowser::RemoveSelectedBookmarkFromFavorites),
		FCanExecuteAction::CreateSP(this, &SWorldBookmarkBrowser::IsValidBookmarkSelected),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &SWorldBookmarkBrowser::IsSelectedBookmarkFavorite));

	Commands->MapAction(FWorldBookmarkCommands::Get().PlayFromLocation,
		FExecuteAction::CreateSP(this, &SWorldBookmarkBrowser::PlayFromSelectedBookmarkLocation),
		FCanExecuteAction::CreateSP(this, &SWorldBookmarkBrowser::CanGoToSelectedBookmarkLocation));

	Commands->MapAction(FWorldBookmarkCommands::Get().MoveCameraToLocation,
		FExecuteAction::CreateSP(this, &SWorldBookmarkBrowser::GoToSelectedBookmarkLocation),
		FCanExecuteAction::CreateSP(this, &SWorldBookmarkBrowser::CanGoToSelectedBookmarkLocation));

	Commands->MapAction(FWorldBookmarkCommands::Get().MoveBookmarkToNewFolder,
		FExecuteAction::CreateSP(this, &SWorldBookmarkBrowser::MoveSelectedBookmarkToNewFolder),
		FCanExecuteAction(),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateLambda([this]() { return GetSelectedBookmark() != nullptr; }));

	Commands->MapAction(FWorldBookmarkCommands::Get().CreateBookmarkInFolder,
		FExecuteAction::CreateSP(this, &SWorldBookmarkBrowser::CreateBookmarkInSelectedFolder),
		FCanExecuteAction(),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateLambda([this]() { return GetSelectedTreeItem() != nullptr; }));

	UpdateDetailsView(/*InSelectedWorldBookmark=*/nullptr);

	RefreshItems(/*bForceRefresh=*/true);

	RecreateColumns();
}

void SWorldBookmarkBrowser::RecreateColumns()
{
	HeaderRow->ClearColumns();

	auto HeaderRowColumn = [this](const Columns::FColumnDefinition& InColumn) -> SHeaderRow::FColumn::FArguments
	{
		return SHeaderRow::Column(InColumn.Id)
			.SortMode(this, &SWorldBookmarkBrowser::GetColumnSortMode, InColumn.Id)
			.SortPriority(this, &SWorldBookmarkBrowser::GetColumnSortPriority, InColumn.Id)
			.OnSort(this, &SWorldBookmarkBrowser::OnColumnSortChanged)
			.DefaultLabel(InColumn.DisplayText)
			.DefaultTooltip(InColumn.ToolTipText);
	};

	auto HeaderRowIconColumn = [this, &HeaderRowColumn](const Columns::FColumnDefinition& InColumn)->SHeaderRow::FColumn::FArguments
	{
		return HeaderRowColumn(InColumn)
			.FixedWidth(24.f)
			.HAlignHeader(HAlign_Left)
			.VAlignHeader(VAlign_Center)
			.HAlignCell(HAlign_Center)
			.VAlignCell(VAlign_Center)
			.HeaderContentPadding(FMargin(4.0f, 0.0f, 0.0f, 0.0f));
	};
	

	// Favorite
	SHeaderRow::FColumn::FArguments FavoriteColumn =
		HeaderRowIconColumn(Columns::Favorite)
		[
			CreateIconWidget(GetFavoriteWorldBookmarkIcon(true))
		];

	// RecentlyUsed
	SHeaderRow::FColumn::FArguments RecentlyUsedColumn =
		HeaderRowIconColumn(Columns::RecentlyUsed)
		[
			CreateIconWidget(GetRecentlyUsedWorldBookmarkIcon())
		];

	// LabelColumn
	SHeaderRow::FColumn::FArguments LabelColumn =
		HeaderRowColumn(Columns::Label)
		.ShouldGenerateWidget(true); // Ensure the column cannot be hidden (grayed out in the show/hide drop down menu)

	// World
	SHeaderRow::FColumn::FArguments WorldColumn =
		HeaderRowColumn(Columns::World)
		.FillWidth(0.65f);

	// Category
	SHeaderRow::FColumn::FArguments CategoryColumn =
		HeaderRowIconColumn(Columns::Category)
		[
			SNew(SColorBlock)
				.Color(FStyleColors::Foreground.GetSpecifiedColor())
				.CornerRadius(FVector4(4.0f, 4.0f, 4.0f, 4.0f))
				.IsEnabled(false)
				.Size(FVector2D(14,14))
		];

	// Order columns differently based on the view mode
	if (UWorldBookmarkBrowserSettings::IsViewMode(EWorldBookmarkBrowserViewMode::TreeView))
	{
		HeaderRow->AddColumn(LabelColumn);
		HeaderRow->AddColumn(WorldColumn);
		HeaderRow->AddColumn(CategoryColumn);
		HeaderRow->AddColumn(FavoriteColumn);
		HeaderRow->AddColumn(RecentlyUsedColumn);
	}
	else
	{
		HeaderRow->AddColumn(FavoriteColumn);
		HeaderRow->AddColumn(RecentlyUsedColumn);
		HeaderRow->AddColumn(CategoryColumn);
		HeaderRow->AddColumn(LabelColumn);
		HeaderRow->AddColumn(WorldColumn);
	}

	// Reapply the hidden columns.
	const TArray<FName>& HiddenColumnsIds = UWorldBookmarkBrowserSettings::GetHiddenColumns();
	for (const SHeaderRow::FColumn& Column : HeaderRow->GetColumns())
	{
		SHeaderRow::FColumn& Col = const_cast<SHeaderRow::FColumn&>(Column);
		Col.bIsVisible = !HiddenColumnsIds.Contains(Column.ColumnId);
	}
	HeaderRow->RefreshColumns();
}

TSharedRef<SWidget> SWorldBookmarkBrowser::GetSettingsMenuContent()
{
	FMenuBuilder MenuBuilder(false, nullptr);

	// ViewType Section
	MenuBuilder.BeginSection(TEXT("SettingsMenuSection_ViewType"), LOCTEXT("SettingsMenuSection_ViewType", "View Type"));
	{
		// View Type -> List View
		MenuBuilder.AddMenuEntry(
			LOCTEXT("Settings_ViewType_ListView", "List"),
			LOCTEXT("Settings_ViewType_ListView_Tooltip", "View bookmarks as a list"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(&UWorldBookmarkBrowserSettings::SetViewMode, EWorldBookmarkBrowserViewMode::ListView),
				FCanExecuteAction(),
				FIsActionChecked::CreateStatic(&UWorldBookmarkBrowserSettings::IsViewMode, EWorldBookmarkBrowserViewMode::ListView)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		// View Type -> Tree View
		MenuBuilder.AddMenuEntry(
			LOCTEXT("Settings_ViewType_TreeView", "Tree"),
			LOCTEXT("Settings_ViewType_TreeView_Tooltip", "View bookmarks as a tree based on the asset path"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(&UWorldBookmarkBrowserSettings::SetViewMode, EWorldBookmarkBrowserViewMode::TreeView),
				FCanExecuteAction(),
				FIsActionChecked::CreateStatic(&UWorldBookmarkBrowserSettings::IsViewMode, EWorldBookmarkBrowserViewMode::TreeView)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}
	MenuBuilder.EndSection();

	// World Bookmark
	MenuBuilder.BeginSection(TEXT("SettingsMenuSection_Show"), LOCTEXT("SettingsMenuSection_Show", "Show"));
	{
		// Show Only For Current World
		MenuBuilder.AddMenuEntry(
			LOCTEXT("Settings_ShowOnlyForCurrentWorld", "Show Only For Current World"),
			LOCTEXT("Settings_ShowOnlyForCurrentWorld_Tooltip", "Displays only bookmarks that are bound to the current world"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(&UWorldBookmarkBrowserSettings::ToggleShowOnlyBookmarksForCurrentWorld),
				FCanExecuteAction(),
				FIsActionChecked::CreateStatic(&UWorldBookmarkBrowserSettings::GetShowOnlyBookmarksForCurrentWorld)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		// Show Only Uncontrolled Bookmarks
		MenuBuilder.AddMenuEntry(
			LOCTEXT("Settings_ShowOnlyUncontrolledBookmarks", "Show Only Uncontrolled Bookmarks"),
			LOCTEXT("Settings_ShowOnlyUncontrolledBookmarks_Tooltip", "Displays only local bookmarks that are kept in an uncontrolled changelist"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(&UWorldBookmarkBrowserSettings::ToggleShowOnlyUncontrolledBookmarks),
				FCanExecuteAction(),
				FIsActionChecked::CreateStatic(&UWorldBookmarkBrowserSettings::GetShowOnlyUncontrolledBookmarks)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		// Show Only Favorite Bookmarks
		MenuBuilder.AddMenuEntry(
			LOCTEXT("Settings_ShowOnlyFavoriteBookmarks", "Show Only Favorite Bookmarks"),
			LOCTEXT("Settings_ShowOnlyFavoriteBookmarks_Tooltip", "Displays only bookmarks that were flagged as favorite"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(&UWorldBookmarkBrowserSettings::ToggleShowOnlyFavoriteBookmarks),
				FCanExecuteAction(),
				FIsActionChecked::CreateStatic(&UWorldBookmarkBrowserSettings::GetShowOnlyFavoriteBookmarks)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		// Show Only Last Recently Used Bookmarks
		MenuBuilder.AddMenuEntry(
			LOCTEXT("Settings_ShowOnlyLastRecentlyUsedBookmarks", "Show Only Last Recently Used Bookmarks"),
			LOCTEXT("Settings_ShowOnlyLastRecentlyUsedBookmarks_Tooltip", "Displays only bookmarks that were recently used"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(&UWorldBookmarkBrowserSettings::ToggleShowOnlyLastRecentlyUsedBookmarks),
				FCanExecuteAction(),
				FIsActionChecked::CreateStatic(&UWorldBookmarkBrowserSettings::GetShowOnlyLastRecentlyUsedBookmarks)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		// Last recently used items
		TSharedRef<SSpinBox<int32>> Slider = SNew(SSpinBox<int32>)
			.MinValue(0)
			.MaxValue(10)
			.ToolTipText(LOCTEXT("Settings_MaxLastRecentlyUsedItems_Tooltip", "Maximum number of last recently used items to display"))
			.Value_Static(&UWorldBookmarkBrowserSettings::GetMaxLastRecentlyUsedItems)
			.OnValueChanged_Static(UWorldBookmarkBrowserSettings::SetMaxLastRecentlyUsedItems);
		
		const bool bNoIndent = true;
		MenuBuilder.AddWidget(Slider, LOCTEXT("Settings_MaxLastRecentlyUsedItems", "Last recently used items"), bNoIndent);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

FReply SWorldBookmarkBrowser::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	bool bIsRenamingItem = false;
	if (FTreeItemPtr SelectedTreeItem = GetSelectedTreeItem())
	{
		bIsRenamingItem = SelectedTreeItem->bInEditingMode;
	}
		
	if (bIsRenamingItem || Commands->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SWorldBookmarkBrowser::RefreshItems(bool bForceRefresh)
{
	// Never perform a refresh outside of the tick method
	if (!bIsInTick && !bForceRefresh)
	{
		bPendingRefresh = true;
		return;
	}

	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
	TArray<FAssetData> AssetsData;
	const bool bSearchSubClasses = true;
	AssetRegistry.GetAssetsByClass(UWorldBookmark::StaticClass()->GetClassPathName(), AssetsData, bSearchSubClasses);

	UWorldBookmark* PreviousSelection = GetSelectedBookmark();

	FSoftObjectPath CurrentWorldPath = FSoftObjectPath(GetCurrentWorld());
	
	TArray<FUncontrolledChangelistStateRef> UncontrolledChangelistsStates;
	if (UWorldBookmarkBrowserSettings::GetShowOnlyUncontrolledBookmarks())
	{
		UncontrolledChangelistsStates = FUncontrolledChangelistsModule::Get().GetChangelistStates();
	}

	// Filter based on "Show Only Bookmarks For Current World"
	auto FilterForCurrentWorld = [&CurrentWorldPath](const FAssetData& AssetData)
	{
		if (UWorldBookmarkBrowserSettings::GetShowOnlyBookmarksForCurrentWorld())
		{
			static const FName NAME_BookmarkWorld(TEXT("WorldName"));

			FString WorldAssetName;
			if (!AssetData.GetTagValue(NAME_BookmarkWorld, WorldAssetName))
			{
				return false;
			}

			FSoftObjectPath WorldAssetPath = FSoftObjectPath(WorldAssetName);
			UAssetRegistryHelpers::FixupRedirectedAssetPath(WorldAssetPath);
			if (CurrentWorldPath != WorldAssetPath)
			{
				return false;
			}
		}

		return true;
	};

	// Filter based on "Show Only Uncontrolled Bookmarks"
	auto FilterUncontrolledBookmarks = [&UncontrolledChangelistsStates](const FAssetData& AssetData)
	{
		if (UWorldBookmarkBrowserSettings::GetShowOnlyUncontrolledBookmarks())
		{
			const FString BookmarkFileName = USourceControlHelpers::PackageFilename(AssetData.PackageName.ToString());
			bool bIsUncontrolledFile = false;

			for (const FUncontrolledChangelistStateRef& UncontrolledChangelistState : UncontrolledChangelistsStates)
			{
				if (UncontrolledChangelistState->GetFilenames().Contains(BookmarkFileName))
				{
					bIsUncontrolledFile = true;
					break;
				}
			}

			if (!bIsUncontrolledFile)
			{
				return false;
			}
		}

		return true;
	};

	// Filter based on "Show Only Favorite Bookmarks"
	auto FilterFavoriteBookmarks = [](const FAssetData& AssetData)
	{
		if (UWorldBookmarkBrowserSettings::GetShowOnlyFavoriteBookmarks())
		{
			UWorldBookmark* WorldBookmark = Cast<UWorldBookmark>(AssetData.GetAsset());
			if (!WorldBookmark->GetIsUserFavorite())
			{
				return false;
			}
		}

		return true;
	};

	// Filter based on "Show Only Last Recently Used Bookmarks"
	auto FilterLRUBookmarks = [](const FAssetData& AssetData)
	{
		if (UWorldBookmarkBrowserSettings::GetShowOnlyLastRecentlyUsedBookmarks())
		{
			UWorldBookmark* WorldBookmark = Cast<UWorldBookmark>(AssetData.GetAsset());
			if (WorldBookmark->GetUserLastLoadedTimeStampUTC() == FDateTime::MinValue())
			{
				return false;
			}
		}

		return true;
	};

	// Filter based on the seach box value
	auto FilterSearch = [this](const FAssetData& AssetData)
	{
		if (!CurrentSearchString.IsEmpty() && !AssetData.AssetName.ToString().Contains(CurrentSearchString.ToString()))
		{
			return false;
		}

		return true;
	};
		
	using FAssetDataFilterPredicate = TFunction<bool(const FAssetData&)>;
	TArray<FAssetDataFilterPredicate> AssetFiltering =
	{
		FilterForCurrentWorld,
		FilterUncontrolledBookmarks,
		FilterFavoriteBookmarks,
		FilterLRUBookmarks,
		FilterSearch
	};

	// Remove all elements which doesn't pass the filters
	AssetsData.RemoveAllSwap([&AssetFiltering](const FAssetData& AssetData)
	{
		// Remove the element if any of the filter return false.
		return Algo::AnyOf(AssetFiltering, [&AssetData](const FAssetDataFilterPredicate& Pred)
		{
			return !Pred(AssetData);
		});
	});

	// Handle special case for LRU items, only keep N items
	if (UWorldBookmarkBrowserSettings::GetShowOnlyLastRecentlyUsedBookmarks() &&
		AssetsData.Num() > UWorldBookmarkBrowserSettings::GetMaxLastRecentlyUsedItems())
	{
		AssetsData.Sort([](const FAssetData& AssetA, const FAssetData& AssetB) 
		{
			UWorldBookmark* WorldBookmarkA = Cast<UWorldBookmark>(AssetA.GetAsset());
			UWorldBookmark* WorldBookmarkB = Cast<UWorldBookmark>(AssetB.GetAsset());
			return WorldBookmarkA->GetUserLastLoadedTimeStampUTC() > WorldBookmarkB->GetUserLastLoadedTimeStampUTC();
		});
		AssetsData.SetNum(UWorldBookmarkBrowserSettings::GetMaxLastRecentlyUsedItems());
	}

	// Recreate the tree items, but retain the expansion state of folders
	TreeRoot->ClearBookmarkItems();
	
	BookmarkTreeItems.Reset();

	for (const FAssetData& AssetData : AssetsData)
	{
		FWorldBookmarkTreeItemPtr WorldBookmarkTreeItem = MakeShared<FWorldBookmarkTreeItem>(AssetData);
		BookmarkTreeItems.Emplace(AssetData.PackageName, WorldBookmarkTreeItem);

		// In flat view mode, add everything to the root
		if (UWorldBookmarkBrowserSettings::GetViewMode() == EWorldBookmarkBrowserViewMode::ListView)
		{
			TreeRoot->AddChild(WorldBookmarkTreeItem);
		}
		else
		{
			FString PackagePath = AssetData.PackagePath.ToString();

			// Extract the mount point
			const FStringView MountPoint = FPathViews::GetMountPointNameFromPath(PackagePath);

			// Convert the mount point to a virtual path (ex: /Game -> /All/Content or /MyPlugin/ -> /All/Plugins/MyPlugin)
			UContentBrowserDataSubsystem* ContentBrowserDataSubsystem = IContentBrowserDataModule::Get().GetSubsystem();			
			FName VirtualPathName;
			ContentBrowserDataSubsystem->ConvertInternalPathToVirtual(FString("/") + MountPoint, VirtualPathName);
		
			// Create the mount point folder
			FFolderTreeItemPtr MountPointFolder = TreeRoot->CreateMountPoint(VirtualPathName.ToString(), FString(MountPoint));
			FString VirtualAssetPath = PackagePath.RightChop(MountPoint.Len() + 1);
			
			// Create the full path to the asset and assign the bookmark as a child
			FFolderTreeItemPtr FolderTreeItem = MountPointFolder->CreatePath(VirtualAssetPath);
			FolderTreeItem->AddChild(WorldBookmarkTreeItem);
		}
	}

	// Now that we have repopulated the tree, do not leave empty folders behind
	TreeRoot->ClearEmptyFolders();
	
	SortItems();

	if (bExpandAllItemsOnNextRefresh || bForceRefresh)
	{
		SetItemExpansionRecursive(TreeRoot, true);
	}

	// Restore selected bookmark after refresh
	SetSelectedBookmark(PreviousSelection);

	bPendingRefresh = false;
	bExpandAllItemsOnNextRefresh = false;
}

void SWorldBookmarkBrowser::SortItems()
{
	auto CompareFolderTreeItems = [this](const FFolderTreeItem* A, const FFolderTreeItem* B) -> bool
	{
		const bool bReverseSort = (ColumnsSortingParams[EColumnSortPriority::Primary].Key == Columns::Label.Id && ColumnsSortingParams[EColumnSortPriority::Primary].Value == EColumnSortMode::Descending) ||
							(ColumnsSortingParams[EColumnSortPriority::Secondary].Key == Columns::Label.Id && ColumnsSortingParams[EColumnSortPriority::Secondary].Value == EColumnSortMode::Descending);

		return bReverseSort ? B->GetName().LexicalLess(A->GetName()) : A->GetName().LexicalLess(B->GetName());
	};

	auto CompareWorldBookmarkTreeItems = [this](const FWorldBookmarkTreeItem* A, const FWorldBookmarkTreeItem* B) -> bool
	{
		auto CompareItems = [&](const TPair<FName, EColumnSortMode::Type>& SortOrder) -> int32
		{
			auto ApplySortMode = [&](int32 SortResult)
			{
				return SortOrder.Value == EColumnSortMode::Ascending ? SortResult : -SortResult;
			};
			
			UWorldBookmark* WorldBookmarkA = Cast<UWorldBookmark>(A->BookmarkAsset.GetAsset());
			UWorldBookmark* WorldBookmarkB = Cast<UWorldBookmark>(B->BookmarkAsset.GetAsset());
			
			if (!WorldBookmarkA || !WorldBookmarkB)
			{
				return WorldBookmarkA ? -1 : 0;
			}

			if (SortOrder.Key == Columns::Favorite.Id)
			{
				return ApplySortMode(WorldBookmarkA->GetIsUserFavorite() == WorldBookmarkB->GetIsUserFavorite() ? 0 : WorldBookmarkA->GetIsUserFavorite() ? -1 : 1);
			}
			else if (SortOrder.Key == Columns::RecentlyUsed.Id)
			{
				return ApplySortMode(WorldBookmarkA->GetUserLastLoadedTimeStampUTC() == WorldBookmarkB->GetUserLastLoadedTimeStampUTC() ? 0 : WorldBookmarkA->GetUserLastLoadedTimeStampUTC() < WorldBookmarkB->GetUserLastLoadedTimeStampUTC() ? -1 : 1);
			}
			else if (SortOrder.Key == Columns::Label.Id)
			{
				return ApplySortMode(WorldBookmarkA->GetName() == WorldBookmarkB->GetName() ? 0 : WorldBookmarkA->GetName() < WorldBookmarkB->GetName() ? -1 : 1);
			}
			else if (SortOrder.Key == Columns::World.Id)
			{
				return ApplySortMode(A->GetBookmarkWorld().GetAssetName() == B->GetBookmarkWorld().GetAssetName() ? 0 : A->GetBookmarkWorld().GetAssetName() < B->GetBookmarkWorld().GetAssetName() ? -1 : 1);
			}
			else if (SortOrder.Key == Columns::Category.Id)
			{
				return ApplySortMode(WorldBookmarkA->GetBookmarkCategory().Name == WorldBookmarkB->GetBookmarkCategory().Name ? 0 : WorldBookmarkA->GetBookmarkCategory() < WorldBookmarkB->GetBookmarkCategory() ? -1 : 1);
			}
		
			checkf(false, TEXT("Unsupported sort order: %s"), *SortOrder.Key.ToString());
			return 0;
		};

		// Primary Sort
		int32 PrimarySortOrder = CompareItems(ColumnsSortingParams[EColumnSortPriority::Primary]);
		if (PrimarySortOrder != 0)
		{
			return PrimarySortOrder < 0;
		}

		// Secondary Sort
		int32 SecondarySortOrder = CompareItems(ColumnsSortingParams[EColumnSortPriority::Secondary]);
		return SecondarySortOrder < 0;
	};

	TreeRoot->Sort([this, &CompareWorldBookmarkTreeItems, &CompareFolderTreeItems](const FTreeItemPtr& LHS, const FTreeItemPtr& RHS)
	{
		if (!LHS.IsValid() || !RHS.IsValid())
		{
			return LHS.IsValid();
		}

		if (LHS->IsA<FFolderTreeItem>() != RHS->IsA<FFolderTreeItem>())
		{
			// Folders should always come first
			return LHS->IsA<FFolderTreeItem>();
		}
		else if (LHS->IsA<FFolderTreeItem>())
		{
			// Sorting two folders
			return CompareFolderTreeItems(LHS->Cast<FFolderTreeItem>(), RHS->Cast<FFolderTreeItem>());
		}
		else
		{
			// Sorting two bookmarks
			return CompareWorldBookmarkTreeItems(LHS->Cast<FWorldBookmarkTreeItem>(), RHS->Cast<FWorldBookmarkTreeItem>());
		}
	});

	if (BookmarksView)
	{
		BookmarksView->RequestListRefresh();
	}
}

const FSoftObjectPath& FWorldBookmarkTreeItem::GetBookmarkWorld() const
{
	if (CachedBookmarkWorld.IsNull())
	{
		CachedBookmarkWorld = UWorldBookmark::GetWorldFromAssetData(BookmarkAsset);
	}	

	return CachedBookmarkWorld;
}

void SWorldBookmarkBrowser::OnGetChildren(FTreeItemPtr InItem, TArray<FTreeItemPtr>& OutChildren)
{
	if (FFolderTreeItem* FolderTreeItem = InItem->Cast<FFolderTreeItem>())
	{
		OutChildren = FolderTreeItem->GetChildren();
	}
}

TSharedRef<ITableRow> SWorldBookmarkBrowser::OnGenerateRow(FTreeItemPtr InTreeItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	if (InTreeItem->IsA<FWorldBookmarkTreeItem>())
	{
		return SNew(FWorldBookmarkTableRow, OwnerTable, StaticCastSharedPtr<FWorldBookmarkTreeItem>(InTreeItem).ToSharedRef());
	}
	else if (InTreeItem->IsA<FFolderTreeItem>())
	{
		return SNew(FFolderTableRow, OwnerTable, StaticCastSharedPtr<FFolderTreeItem>(InTreeItem).ToSharedRef());
	}

	return SNew(STableRow<TSharedPtr<FString>>, OwnerTable);
}

void SWorldBookmarkBrowser::OnSelectionChanged(FTreeItemPtr InTreeItem, ESelectInfo::Type SelectionType)
{
	UWorldBookmark* SelectedWorldBookmark = GetSelectedBookmark();
	PropertyView->SetObject(SelectedWorldBookmark);
	UpdateDetailsView(SelectedWorldBookmark);
}

void SWorldBookmarkBrowser::OnMouseButtonDoubleClicked(FTreeItemPtr InTreeItem)
{
	if (UWorldBookmark* WorldBookmark = GetSelectedBookmark())
	{
		if (!AskForWorldChangeConfirmation(WorldBookmark))
		{
			return;
		}

		WorldBookmark->Load();

		// Loading an item will update it's last load time - refresh the UI
		RefreshItems();
	}
}

FReply SWorldBookmarkBrowser::OnCreateNewWorldBookmarkClicked()
{
	FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();
	UWorldBookmarkFactory* Factory = NewObject<UWorldBookmarkFactory>();

	// Create a new bookmark asset
	UObject* NewAsset = AssetToolsModule.Get().CreateAssetWithDialog(UWorldBookmark::StaticClass(), Factory);
	if (UWorldBookmark* NewWorldBookmark = Cast<UWorldBookmark>(NewAsset))
	{
		// Show it in the content browser
		GEditor->SyncBrowserToObject(NewWorldBookmark);

		// Force refresh the bookmark browser, to ensure the new bookmark tree item is created
		RefreshItems(/*bForceRefresh=*/true);

		// Make sure the new bookmark is selected
		SetSelectedBookmark(NewWorldBookmark);
	}

	return FReply::Handled();
}

void SWorldBookmarkBrowser::OnSearchBoxTextChanged(const FText& InSearchText)
{
	if (!CurrentSearchString.EqualTo(InSearchText))
	{
		CurrentSearchString = InSearchText;
		bExpandAllItemsOnNextRefresh = true;
		RefreshItems();
	}
}

EColumnSortMode::Type SWorldBookmarkBrowser::GetColumnSortMode(const FName InColumnId) const
{
	if (ColumnsSortingParams[EColumnSortPriority::Primary].Key == InColumnId)
	{
		return ColumnsSortingParams[EColumnSortPriority::Primary].Value;
	}

	if (ColumnsSortingParams[EColumnSortPriority::Secondary].Key == InColumnId)
	{
		return ColumnsSortingParams[EColumnSortPriority::Secondary].Value;
	}

	return EColumnSortMode::None;
}

EColumnSortPriority::Type SWorldBookmarkBrowser::GetColumnSortPriority(const FName InColumnId) const
{
	if (ColumnsSortingParams[EColumnSortPriority::Primary].Key == InColumnId)
	{
		return EColumnSortPriority::Primary;
	}

	if (ColumnsSortingParams[EColumnSortPriority::Secondary].Key == InColumnId)
	{
		return EColumnSortPriority::Secondary;
	}

	return EColumnSortPriority::None;
}

void SWorldBookmarkBrowser::OnColumnSortChanged(const EColumnSortPriority::Type InSortPriority, const FName& InColumnId, const EColumnSortMode::Type InSortMode)
{
	ColumnsSortingParams[InSortPriority].Key = InColumnId;
	ColumnsSortingParams[InSortPriority].Value = InSortMode;

	SortItems();
}

TSharedPtr<SWidget> SWorldBookmarkBrowser::OnContextMenuOpening()
{
	FMenuBuilder MenuBuilder(true, Commands);

	FTreeItemPtr SelectedItem = GetSelectedTreeItem();
	FWorldBookmarkTreeItem* BookmarkTreeItem = SelectedItem.IsValid() ? SelectedItem->Cast<FWorldBookmarkTreeItem>() : nullptr;
	FFolderTreeItem* FolderTreeItem = SelectedItem.IsValid() ? SelectedItem->Cast<FFolderTreeItem>() : nullptr;

	// No context menu for virtual folders
	if (FolderTreeItem && FolderTreeItem->IsVirtual())
	{
		return SNullWidget::NullWidget;
	}

	// World Bookmark
	if (BookmarkTreeItem)
	{		
		MenuBuilder.BeginSection(TEXT("WorldBookmarkMenuSection"), LOCTEXT("WorldBookmarkMenuSection_Label", "World Bookmark"));
		MenuBuilder.AddMenuEntry(FWorldBookmarkCommands::Get().LoadBookmark);
		MenuBuilder.AddMenuEntry(FWorldBookmarkCommands::Get().UpdateBookmark);
		MenuBuilder.AddMenuEntry(FWorldBookmarkCommands::Get().AddToFavorite);
		MenuBuilder.AddMenuEntry(FWorldBookmarkCommands::Get().RemoveFromFavorite);
		MenuBuilder.AddMenuEntry(FWorldBookmarkCommands::Get().PlayFromLocation);
		MenuBuilder.AddMenuEntry(FWorldBookmarkCommands::Get().MoveCameraToLocation);
		MenuBuilder.EndSection();
	}

	// Asset/Folder Options
	if (FolderTreeItem)
	{
		MenuBuilder.BeginSection(TEXT("FolderOptionsSection"), LOCTEXT("FolderOptionsText", "Folder Options"));
	}
	else
	{
		MenuBuilder.BeginSection(TEXT("AssetOptionsSection"), LOCTEXT("AssetOptionsText", "Asset Options"));
	}
	{
		// Find In Content Browser
		MenuBuilder.AddMenuEntry(FGlobalEditorCommonCommands::Get().FindInContentBrowser);

		// Rename
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename);

		// Delete
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);

		if (FolderTreeItem)
		{
			MenuBuilder.AddMenuEntry(FWorldBookmarkCommands::Get().CreateBookmarkInFolder);
		}
		else
		{
			MenuBuilder.AddMenuEntry(FWorldBookmarkCommands::Get().MoveBookmarkToNewFolder);
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

static bool IsWorldBookmarkAsset(const FAssetData& InAssetData)
{
	return InAssetData.AssetClassPath == UWorldBookmark::StaticClass()->GetClassPathName();
}

static bool IsWorldBookmarkAsset(UObject* InObject)
{
	return InObject->IsA<UWorldBookmark>();
}

void SWorldBookmarkBrowser::OnAssetsAdded(TConstArrayView<FAssetData> InAssets)
{
	for (const FAssetData& InAssetData : InAssets)
	{
		if (IsWorldBookmarkAsset(InAssetData))
		{
			RefreshItems();
		}
	}
}

void SWorldBookmarkBrowser::OnAssetRemoved(const FAssetData& InAssetData)
{
	if (IsWorldBookmarkAsset(InAssetData))
	{
		RefreshItems();
	}
}

void SWorldBookmarkBrowser::OnAssetRenamed(const FAssetData& InAssetData, const FString& InOldObjectPath)
{
	if (IsWorldBookmarkAsset(InAssetData))
	{
		if (FWorldBookmarkTreeItemPtr SelectedBookmarkTreeItem = GetSelectedBookmarkTreeItem())
		{
			// If the selected tree item is the one that was renamed, assign the new AssetData to it
			// This will allow it to remain selected after the view refreshes.
			if (SelectedBookmarkTreeItem->BookmarkAsset.GetObjectPathString() == InOldObjectPath)
			{
				SelectedBookmarkTreeItem->BookmarkAsset = InAssetData;
			}
		}

		RefreshItems();
	}
}

void SWorldBookmarkBrowser::OnAssetUpdated(const FAssetData& InAssetData)
{
	if (IsWorldBookmarkAsset(InAssetData))
	{
		RefreshItems();
	}
}

void SWorldBookmarkBrowser::OnAssetUpdatedOnDisk(const FAssetData& InAssetData)
{
	if (IsWorldBookmarkAsset(InAssetData))
	{
		RefreshItems();
	}
}

void SWorldBookmarkBrowser::OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (IsWorldBookmarkAsset(InObject))
	{
		RefreshItems();
	}
}

void SWorldBookmarkBrowser::OnObjectPreSave(UObject* InObject, FObjectPreSaveContext InObjectPreSaveContext)
{
	if (IsWorldBookmarkAsset(InObject))
	{
		RefreshItems();
	}
}

void SWorldBookmarkBrowser::OnMapChanged(UWorld* World, EMapChangeType MapChangeType)
{
	RefreshItems();
}

void SWorldBookmarkBrowser::OnSettingsChanged()
{
	if (UWorldBookmarkBrowserSettings::GetShowOnlyUncontrolledBookmarks())
	{
		if (!OnUncontrolledChangelistModuleChangedHandle.IsValid())
		{
			OnUncontrolledChangelistModuleChangedHandle = FUncontrolledChangelistsModule::Get().OnUncontrolledChangelistModuleChanged.AddRaw(this, &SWorldBookmarkBrowser::OnUncontrolledChangelistModuleChanged);
		}
	}
	else if (OnUncontrolledChangelistModuleChangedHandle.IsValid())
	{
		FUncontrolledChangelistsModule::Get().OnUncontrolledChangelistModuleChanged.Remove(OnUncontrolledChangelistModuleChangedHandle);
		OnUncontrolledChangelistModuleChangedHandle.Reset();
	}

	RecreateColumns();
	RefreshItems();
}

void SWorldBookmarkBrowser::OnUncontrolledChangelistModuleChanged()
{
	RefreshItems();
}

bool SWorldBookmarkBrowser::CanExecuteBookmarkAction() const
{
	// Disable actions in PIE
	if (GEditor->PlayWorld != nullptr || GEditor->bIsSimulatingInEditor)
	{
		return false;
	}

	// Disable actions if no bookmark is selected
	return IsValidBookmarkSelected();
}

void SWorldBookmarkBrowser::LoadSelectedBookmark()
{
	if (UWorldBookmark* SelectedBookmark = GetSelectedBookmark())
	{
		if (!AskForWorldChangeConfirmation(SelectedBookmark))
		{
			return;
		}

		SelectedBookmark->Load();
	}
}

void SWorldBookmarkBrowser::UpdateSelectedBookmark()
{
	if (UWorldBookmark* SelectedBookmark = GetSelectedBookmark())
	{
		SelectedBookmark->Update();
		PropertyView->ForceRefresh();
	}
}

void SWorldBookmarkBrowser::FindSelectedItemInContentBrowser()
{
	if (FTreeItemPtr SelectedTreeItem = GetSelectedTreeItem())
	{
		SelectedTreeItem->ShowInContentBrowser();
	}
}

void SWorldBookmarkBrowser::AddSelectedBookmarkToFavorites()
{
	if (UWorldBookmark* SelectedBookmark = GetSelectedBookmark())
	{
		SelectedBookmark->SetIsUserFavorite(true);
		RefreshItems();
	}
}

void SWorldBookmarkBrowser::RemoveSelectedBookmarkFromFavorites()
{
	if (UWorldBookmark* SelectedBookmark = GetSelectedBookmark())
	{
		SelectedBookmark->SetIsUserFavorite(false);
		RefreshItems();
	}
}

bool SWorldBookmarkBrowser::IsSelectedBookmarkFavorite() const
{
	if (UWorldBookmark* SelectedBookmark = GetSelectedBookmark())
	{
		return SelectedBookmark->GetIsUserFavorite();
	}

	return false;
}

bool SWorldBookmarkBrowser::IsSelectedBookmarkNotFavorite() const
{
	if (UWorldBookmark* SelectedBookmark = GetSelectedBookmark())
	{
		return !SelectedBookmark->GetIsUserFavorite();
	}

	return true;
}

bool SWorldBookmarkBrowser::IsValidBookmarkSelected() const
{
	return GetSelectedBookmark() != nullptr;
}

FTreeItemPtr SWorldBookmarkBrowser::GetSelectedTreeItem() const
{
	if (!BookmarksView)
	{
		return nullptr;
	}

	const TArray<FTreeItemPtr> Selection = BookmarksView->GetSelectedItems();
	if (Selection.Num() != 1)
	{
		return nullptr;
	}

	return Selection[0];
}

bool SWorldBookmarkBrowser::IsValidItemSelected() const
{
	return GetSelectedTreeItem() != nullptr;
}

FWorldBookmarkTreeItemPtr SWorldBookmarkBrowser::GetSelectedBookmarkTreeItem() const
{
	FTreeItemPtr SelectedTreeItem = GetSelectedTreeItem();
	if (!SelectedTreeItem || !SelectedTreeItem->IsA<FWorldBookmarkTreeItem>())
	{
		return nullptr;
	}

	FWorldBookmarkTreeItemPtr SelectedWorldBookmarkTreeItem = StaticCastSharedPtr<FWorldBookmarkTreeItem>(SelectedTreeItem);
	if (!SelectedWorldBookmarkTreeItem->BookmarkAsset.IsValid())
	{
		return nullptr;
	}

	return SelectedWorldBookmarkTreeItem;
}

UWorldBookmark* SWorldBookmarkBrowser::GetSelectedBookmark() const
{
	FWorldBookmarkTreeItemPtr SelectedItem = GetSelectedBookmarkTreeItem();
	if (!SelectedItem)
	{
		return nullptr;
	}

	return Cast<UWorldBookmark>(SelectedItem->BookmarkAsset.GetAsset());
}

void SWorldBookmarkBrowser::SetSelectedBookmark(UWorldBookmark* InBookmark)
{
	if (!BookmarksView)
	{
		return;
	}

	if (InBookmark)
	{
		FWorldBookmarkTreeItemPtr* BookmarkTreeItem = BookmarkTreeItems.Find(InBookmark->GetPackage()->GetFName());
		if (BookmarkTreeItem)
		{
			BookmarksView->SetSelection(*BookmarkTreeItem);
			BookmarksView->RequestNavigateToItem(*BookmarkTreeItem);
		}
	}
	else
	{
		BookmarksView->ClearSelection();
	}
}

void SWorldBookmarkBrowser::OnItemScrolledIntoView(FTreeItemPtr InTreeItem, const TSharedPtr<ITableRow>& InWidget)
{
	if (TreeItemPendingRename && InTreeItem == TreeItemPendingRename)
	{
		// Abort the rename if our widget has lost focus.
		TSharedPtr<SWindow> OwnerWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		if (OwnerWindow && OwnerWindow->HasAnyUserFocusOrFocusedDescendants())
		{
			// We know the tree item is now visible, bring up the editable text block.
			TreeItemPendingRename->OnRenameRequested.ExecuteIfBound();
		}

		TreeItemPendingRename.Reset();
	}
}

void SWorldBookmarkBrowser::RequestRenameSelectedItem()
{
	if (FTreeItemPtr SelectedTreeItem = GetSelectedTreeItem())
	{
		// Before showing the inline editable text block, we must ensure that the tree item is visible.
		TreeItemPendingRename = SelectedTreeItem;
		BookmarksView->RequestScrollIntoView(SelectedTreeItem);
	}
}

bool SWorldBookmarkBrowser::CanRenameOrDeleteSelectedItem() const
{
	if (FTreeItemPtr SelectedTreeItem = GetSelectedTreeItem())
	{
		return SelectedTreeItem->CanRename();
	}

	return false;
}

void SWorldBookmarkBrowser::DeleteSelectedItem()
{
	if (FTreeItemPtr SelectedTreeItem = GetSelectedTreeItem())
	{
		SelectedTreeItem->Delete();
	}
}

bool SWorldBookmarkBrowser::AskForWorldChangeConfirmation(UWorldBookmark* WorldBookmark) const
{
	if (const UWorldEditorState* WorldEditorState = WorldBookmark->GetEditorState<UWorldEditorState>())
	{
		TSoftObjectPtr<UWorld> BookmarkWorld = WorldEditorState->GetStateWorld();
		if (BookmarkWorld != GetCurrentWorld())
		{
			const FText MessageBoxTitle = LOCTEXT("WorldBookmark_DifferentWorldDlg_Title", "Open Bookmark World?");
			const FText MessageBoxText = LOCTEXT("WorldBookmark_DifferentWorldDlg_Text", "This bookmark is for a different world, are you sure you want to open it?");
			
			FSuppressableWarningDialog::FSetupInfo Info(MessageBoxText, MessageBoxTitle, "WorldBookmark_AskForWorldChangeConfirmation");
			Info.ConfirmText = FCoreTexts::Get().Yes;
			Info.CancelText = FCoreTexts::Get().No;

			FSuppressableWarningDialog AddLevelWarning(Info);
			const FSuppressableWarningDialog::EResult Response = AddLevelWarning.ShowModal();
			return Response != FSuppressableWarningDialog::EResult::Cancel;
		}
	}

	return true;
}

bool SWorldBookmarkBrowser::CanGoToSelectedBookmarkLocation() const
{
	if (!CanExecuteBookmarkAction())
	{
		return false;
	}

	if (UWorldBookmark* SelectedBookmark = GetSelectedBookmark())
	{
		if (SelectedBookmark->HasEditorState<ULevelEditorCameraEditorState>())
		{
			return true;
		}
	}

	return false;
}

void SWorldBookmarkBrowser::PlayFromSelectedBookmarkLocation()
{
	if (UWorldBookmark* SelectedBookmark = GetSelectedBookmark())
	{
		// Test if state contains LevelEditorCameraState ?
		if (SelectedBookmark->HasEditorState<ULevelEditorCameraEditorState>())
		{
			if (!AskForWorldChangeConfirmation(SelectedBookmark))
			{
				return;
			}

			SelectedBookmark->LoadStates({ UWorldEditorState::StaticClass() });

			if (const ULevelEditorCameraEditorState* CameraEditorState = SelectedBookmark->GetEditorState<ULevelEditorCameraEditorState>())
			{
				FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
				TSharedPtr<IAssetViewport> ActiveViewport = LevelEditorModule.GetFirstActiveViewport();
				FPlayWorldCommandCallbacks::StartPlayFromHere(CameraEditorState->GetCameraLocation(), CameraEditorState->GetCameraRotation(), ActiveViewport);
			}
		}
	}
}

void SWorldBookmarkBrowser::GoToSelectedBookmarkLocation()
{
	if (UWorldBookmark* SelectedBookmark = GetSelectedBookmark())
	{
		// Test if state contains LevelEditorCameraState ?
		if (SelectedBookmark->HasEditorState<ULevelEditorCameraEditorState>())
		{
			if (!AskForWorldChangeConfirmation(SelectedBookmark))
			{
				return;
			}

			SelectedBookmark->LoadStates({ ULevelEditorCameraEditorState::StaticClass() });
		}
	}
}

void SWorldBookmarkBrowser::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	TGuardValue<bool> IsInTickGuard(bIsInTick, true);

	if (bPendingRefresh)
	{
		RefreshItems();
	}

	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

void SWorldBookmarkBrowser::OnItemExpansionChanged(FTreeItemPtr InTreeItem, bool bIsExpanded)
{
	if (FFolderTreeItem* FolderItem = InTreeItem->Cast<FFolderTreeItem>())
	{
		FolderItem->SetExpanded(bIsExpanded);

		// Expand any children that are also expanded
		FolderItem->ForEachChildRecursive<FFolderTreeItem>([this](FFolderTreeItemPtr ChildFolderItem)
		{
			if (ChildFolderItem->IsExpanded())
			{
				BookmarksView->SetItemExpansion(ChildFolderItem, true);
			}
		});
	}
}

void SWorldBookmarkBrowser::SetItemExpansionRecursive(FTreeItemPtr InTreeItem, bool bIsExpanded)
{
	if (FFolderTreeItem* FolderItem = InTreeItem->Cast<FFolderTreeItem>())
	{
		BookmarksView->SetItemExpansion(InTreeItem, bIsExpanded);

		// Expand any children
		FolderItem->ForEachChildRecursive<FFolderTreeItem>([this](FFolderTreeItemPtr ChildFolderItem)
		{
			BookmarksView->SetItemExpansion(ChildFolderItem, true);
		});
	}
}

void SWorldBookmarkBrowser::UpdateDetailsView(UWorldBookmark* SelectedWorldBookmark)
{
	DetailsBox->ClearChildren();

	// Show message if no bookmark is selected
	if (SelectedWorldBookmark == nullptr)
	{
		DetailsBox->AddSlot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Top)
			.Padding(2.0f, 24.0f, 2.0f, 2.0f)
			[
				SNew(STextBlock)
					.Text(LOCTEXT("NoBookmarkSelected", "Select a world bookmark to view details."))
					.TextStyle(FAppStyle::Get(), "HintText")
			];
	}
	// Show message if bookmark has no data
	else if (!SelectedWorldBookmark->HasEditorStates())
	{
		DetailsBox->AddSlot()
			.HAlign(HAlign_Center)
			.Padding(2.0f, 24.0f, 2.0f, 2.0f)
			[
				SNew(STextBlock)
					.Text(LOCTEXT("UninitializedBookmarkSelected", "Bookmark is empty. Update it to view details."))
					.TextStyle(FAppStyle::Get(), "HintText")
			];
	}
	// Show details property
	else
	{
		DetailsBox->AddSlot()
			.Padding(2, 4, 0, 0)
			[
				PropertyView.ToSharedRef()
			];
	}
}

void SWorldBookmarkBrowser::MoveSelectedBookmarkToNewFolder()
{
	FTreeItemPtr ItemToMove = GetSelectedTreeItem();
	if (ItemToMove)
	{
		FFolderTreeItem* ParentFolder = ItemToMove->GetParent()->Cast<FFolderTreeItem>();
		if (ParentFolder)
		{
			bool bValidName = true;
			FString BaseFolderName(TEXT("NewFolder"));
			FString NewFolderName(BaseFolderName);
			int32 FolderCounter = 0;
			do
			{
				bValidName = true;
				ParentFolder->ForEachChild<FFolderTreeItem>([NewFolderName, &bValidName](FFolderTreeItemPtr ChildFolder)
				{
					bValidName &= ChildFolder->GetName() == NewFolderName;
				});

				if (!bValidName)
				{
					NewFolderName = FString::Printf(TEXT("%s%d"), *BaseFolderName, FolderCounter);
					FolderCounter++;
				}
			} while (!bValidName);

			FFolderTreeItemPtr NewFolder = ParentFolder->CreatePath(NewFolderName);

			NewFolder->Move(ItemToMove);
			BookmarksView->SetSelection(NewFolder);
			RequestRenameSelectedItem();
		}
	}
}

void SWorldBookmarkBrowser::CreateBookmarkInSelectedFolder()
{
	if (FTreeItemPtr SelectedTreeItem = GetSelectedTreeItem())
	{
		if (FFolderTreeItem* SelectedFolder = SelectedTreeItem->Cast<FFolderTreeItem>())
		{
			FString AssetPath = SelectedFolder->GetAssetPath();
			
			FString AssetName;
			FString PackageName;

			IAssetTools* AssetTools = &FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();
			AssetTools->CreateUniqueAssetName(AssetPath / TEXT("NewWorldBookmark"), FString(), PackageName, AssetName);
			UObject* NewAsset = AssetTools->CreateAsset(AssetName, AssetPath, UWorldBookmark::StaticClass(), nullptr);

			if (UWorldBookmark* NewWorldBookmark = Cast<UWorldBookmark>(NewAsset))
			{
				// Show it in the content browser
				GEditor->SyncBrowserToObject(NewWorldBookmark);

				// Force refresh the bookmark browser, to ensure the new bookmark tree item is created
				RefreshItems(/*bForceRefresh=*/true);

				// Make sure the new bookmark is selected
				SetSelectedBookmark(NewWorldBookmark);

				// Enter editing mode on the new bookmark label
				RequestRenameSelectedItem();
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE

}
