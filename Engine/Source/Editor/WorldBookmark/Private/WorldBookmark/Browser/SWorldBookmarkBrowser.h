// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AssetTextFilter.h"
#include "Misc/TextFilter.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectSaveContext.h"
#include "UnrealEdMisc.h"
#include "Widgets/Views/STreeView.h"

#include "WorldBookmark/Browser/FolderTreeItem.h"
#include "WorldBookmark/Browser/BookmarkTreeItem.h"

class UWorldBookmark;
class IDetailsView;
class SAssetView;

namespace UE::WorldBookmark::Browser
{

class SWorldBookmarkBrowser : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SWorldBookmarkBrowser)
		{}
	SLATE_END_ARGS()

	SWorldBookmarkBrowser();
	virtual ~SWorldBookmarkBrowser();

	void Construct(const FArguments& InArgs);

private:
	// ~Begin STreeView delegates
	void OnGetChildren(FTreeItemPtr InItem, TArray<FTreeItemPtr>& OutChildren);
	TSharedRef<ITableRow> OnGenerateRow(FTreeItemPtr InTreeItem, const TSharedRef<STableViewBase>& OwnerTable);
	void OnSelectionChanged(FTreeItemPtr InTreeItem, ESelectInfo::Type SelectionType);
	void OnMouseButtonDoubleClicked(FTreeItemPtr InTreeItem);
	TSharedPtr<SWidget> OnContextMenuOpening();
	void OnItemScrolledIntoView(FTreeItemPtr WorldBookmarkTreeItem, const TSharedPtr<ITableRow>& Widget);
	// ~End STreeView delegates

	FReply OnCreateNewWorldBookmarkClicked();
	void OnSearchBoxTextChanged(const FText& InSearchText);
	void OnColumnSortChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode);
	TSharedRef<SWidget> GetSettingsMenuContent();

	/** SWidget interface */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	void RefreshItems(bool bForceRefresh = false);
	void SortItems();

	void RecreateColumns();
	EColumnSortMode::Type GetColumnSortMode(const FName ColumnId) const;
	EColumnSortPriority::Type GetColumnSortPriority(const FName InColumnId) const;
	TArray<FAssetViewCustomColumn> GetCustomColumns() const;
	
	void OnAssetsAdded(TConstArrayView<FAssetData> InAssets);
	void OnAssetRemoved(const FAssetData& InAssetData);
	void OnAssetRenamed(const FAssetData& InAssetData, const FString& InOldObjectPath);
	void OnAssetUpdated(const FAssetData& InAssetData);
	void OnAssetUpdatedOnDisk(const FAssetData& InAssetData);
	void OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent);
	void OnObjectPreSave(UObject* InObject, FObjectPreSaveContext InObjectPreSaveContext);

	void OnMapChanged(UWorld* World, EMapChangeType MapChangeType);
	
	void OnSettingsChanged();

	void OnUncontrolledChangelistModuleChanged();

	FTreeItemPtr GetSelectedTreeItem() const;
	FWorldBookmarkTreeItemPtr GetSelectedBookmarkTreeItem() const;
	UWorldBookmark* GetSelectedBookmark() const;
	void SetSelectedBookmark(UWorldBookmark* InBookmark);
	bool IsValidBookmarkSelected() const;
	bool IsValidItemSelected() const;

	bool CanExecuteBookmarkAction() const;

	void LoadSelectedBookmark();
	void UpdateSelectedBookmark();

	bool IsSelectedBookmarkFavorite() const;
	bool IsSelectedBookmarkNotFavorite() const;
	void AddSelectedBookmarkToFavorites();
	void RemoveSelectedBookmarkFromFavorites();	

	void FindSelectedItemInContentBrowser();
	void RequestRenameSelectedItem();

	bool CanRenameOrDeleteSelectedItem() const;
	void DeleteSelectedItem();

	bool CanGoToSelectedBookmarkLocation() const;
	void PlayFromSelectedBookmarkLocation();
	void GoToSelectedBookmarkLocation();

	void UpdateDetailsView(UWorldBookmark* InSelectedWorldBookmark);

	void OnItemExpansionChanged(FTreeItemPtr InTreeItem, bool bIsExpanded);
	void SetItemExpansionRecursive(FTreeItemPtr InTreeItem, bool bIsExpanded);

	void MoveSelectedBookmarkToNewFolder();
	void CreateBookmarkInSelectedFolder();

	bool AskForWorldChangeConfirmation(UWorldBookmark* WorldBookmark) const;

private:
	FDelegateHandle OnAssetAddedHandle;
	FDelegateHandle OnAssetRemovedHandle;
	FDelegateHandle OnAssetRenamedHandle;
	FDelegateHandle OnAssetUpdatedHandle;
	FDelegateHandle OnAssetUpdatedOnDiskHandle;
	FDelegateHandle OnObjectPropertyChangedHandle;
	FDelegateHandle OnMapChangedHandle;
	FDelegateHandle OnSettingsChangedHandle;
	FDelegateHandle OnUncontrolledChangelistModuleChangedHandle;

	FFolderTreeItemPtr TreeRoot;
	TMap<FName, FWorldBookmarkTreeItemPtr> BookmarkTreeItems;

	TSharedPtr<SHeaderRow> HeaderRow;
	TSharedPtr<STreeView<FTreeItemPtr>> BookmarksView;
	TSharedPtr<IDetailsView> PropertyView;
	TSharedPtr<SVerticalBox> DetailsBox;

	TSharedPtr<FUICommandList> Commands;

	TPair<FName, EColumnSortMode::Type> ColumnsSortingParams[EColumnSortPriority::Max];
	
	FText CurrentSearchString;
	
	FTreeItemPtr TreeItemPendingRename;

	bool bIsInTick;
	bool bPendingRefresh;
	bool bExpandAllItemsOnNextRefresh;
};

}
