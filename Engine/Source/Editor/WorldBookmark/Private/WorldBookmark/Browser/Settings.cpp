// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldBookmark/Browser/Settings.h"

#include "WorldBookmark/Browser/Columns.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Settings)

TObjectPtr<UWorldBookmarkBrowserSettings> UWorldBookmarkBrowserSettings::Instance = nullptr;

UWorldBookmarkBrowserSettings& UWorldBookmarkBrowserSettings::Get()
{
	if (!Instance)
	{
		Instance = NewObject<UWorldBookmarkBrowserSettings>();

		Instance->HiddenColumns = UE::WorldBookmark::Browser::Columns::DefaultHiddenColumns;

		Instance->LoadEditorConfig();
		Instance->AddToRoot();
	}

	return *Instance;
}

void UWorldBookmarkBrowserSettings::ApplySettingsChanges()
{
	SaveEditorConfig();
	OnSettingsChangedDelegate.Broadcast();
}

UWorldBookmarkBrowserSettings::FWorldBookmarkBrowserSettingChanged& UWorldBookmarkBrowserSettings::OnSettingsChanged()
{
	UWorldBookmarkBrowserSettings& Settings = Get();
	return Settings.OnSettingsChangedDelegate;
}

bool UWorldBookmarkBrowserSettings::GetShowOnlyBookmarksForCurrentWorld()
{
	return Get().bShowOnlyBookmarksForCurrentWorld;
}

void UWorldBookmarkBrowserSettings::ToggleShowOnlyBookmarksForCurrentWorld()
{
	UWorldBookmarkBrowserSettings& Settings = Get();
	Settings.bShowOnlyBookmarksForCurrentWorld = !Settings.bShowOnlyBookmarksForCurrentWorld;
	Settings.ApplySettingsChanges();
}

bool UWorldBookmarkBrowserSettings::GetShowOnlyUncontrolledBookmarks()
{
	return Get().bShowOnlyUncontrolledBookmarks;
}

void UWorldBookmarkBrowserSettings::ToggleShowOnlyUncontrolledBookmarks()
{
	UWorldBookmarkBrowserSettings& Settings = Get();
	Settings.bShowOnlyUncontrolledBookmarks = !Settings.bShowOnlyUncontrolledBookmarks;
	Settings.ApplySettingsChanges();
}

bool UWorldBookmarkBrowserSettings::GetShowOnlyFavoriteBookmarks()
{
	return Get().bShowOnlyFavoriteBookmarks;
}

void UWorldBookmarkBrowserSettings::ToggleShowOnlyFavoriteBookmarks()
{
	UWorldBookmarkBrowserSettings& Settings = Get();
	Settings.bShowOnlyFavoriteBookmarks = !Settings.bShowOnlyFavoriteBookmarks;
	Settings.ApplySettingsChanges();
}

bool UWorldBookmarkBrowserSettings::GetShowOnlyLastRecentlyUsedBookmarks()
{
	return Get().bShowOnlyLastRecentlyUsedBookmarks;
}

void UWorldBookmarkBrowserSettings::ToggleShowOnlyLastRecentlyUsedBookmarks()
{
	UWorldBookmarkBrowserSettings& Settings = Get();
	Settings.bShowOnlyLastRecentlyUsedBookmarks = !Settings.bShowOnlyLastRecentlyUsedBookmarks;
	Settings.ApplySettingsChanges();
}

TArray<FName> UWorldBookmarkBrowserSettings::GetHiddenColumns()
{
	return Get().HiddenColumns;
}

void UWorldBookmarkBrowserSettings::SetHiddenColumns(const TArray<FName>& InHiddenColumns)
{
	UWorldBookmarkBrowserSettings& Settings = Get();
	if (Settings.HiddenColumns != InHiddenColumns)
	{
		Settings.HiddenColumns = InHiddenColumns;
		Settings.ApplySettingsChanges();
	}
}

int32 UWorldBookmarkBrowserSettings::GetMaxLastRecentlyUsedItems()
{
	return Get().MaxLastRecentlyUsedItems;
}

void UWorldBookmarkBrowserSettings::SetMaxLastRecentlyUsedItems(int32 InMaxLastRecentlyUsedItems)
{
	UWorldBookmarkBrowserSettings& Settings = Get();
	if (Settings.MaxLastRecentlyUsedItems != InMaxLastRecentlyUsedItems)
	{
		Settings.MaxLastRecentlyUsedItems = InMaxLastRecentlyUsedItems;
		Settings.ApplySettingsChanges();
	}
}

EWorldBookmarkBrowserViewMode UWorldBookmarkBrowserSettings::GetViewMode()
{
	return Get().ViewMode;
}

bool UWorldBookmarkBrowserSettings::IsViewMode(EWorldBookmarkBrowserViewMode InViewMode)
{
	return GetViewMode() == InViewMode;
}

void UWorldBookmarkBrowserSettings::SetViewMode(EWorldBookmarkBrowserViewMode InViewMode)
{
	UWorldBookmarkBrowserSettings& Settings = Get();
	if (Settings.ViewMode != InViewMode)
	{
		Settings.ViewMode = InViewMode;
		Settings.ApplySettingsChanges();
	}
}
