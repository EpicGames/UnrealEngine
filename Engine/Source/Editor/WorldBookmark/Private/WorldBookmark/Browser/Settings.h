// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "EditorConfigBase.h"

#include "Settings.generated.h"

UENUM()
enum class EWorldBookmarkBrowserViewMode : uint8
{
	ListView,
	TreeView
};

UCLASS(EditorConfig = "WorldBookmarkBrowser")
class UWorldBookmarkBrowserSettings : public UEditorConfigBase
{
	GENERATED_BODY()

public:
	DECLARE_MULTICAST_DELEGATE(FWorldBookmarkBrowserSettingChanged)

	static FWorldBookmarkBrowserSettingChanged& OnSettingsChanged();

	static bool GetShowOnlyBookmarksForCurrentWorld();
	static void ToggleShowOnlyBookmarksForCurrentWorld();

	static bool GetShowOnlyUncontrolledBookmarks();
	static void ToggleShowOnlyUncontrolledBookmarks();

	static bool GetShowOnlyFavoriteBookmarks();
	static void ToggleShowOnlyFavoriteBookmarks();

	static bool GetShowOnlyLastRecentlyUsedBookmarks();
	static void ToggleShowOnlyLastRecentlyUsedBookmarks();

	static TArray<FName> GetHiddenColumns();
	static void SetHiddenColumns(const TArray<FName>& InHiddenColumns);

	static int32 GetMaxLastRecentlyUsedItems();
	static void SetMaxLastRecentlyUsedItems(int32 InMaxLastRecentlyUsedItems);

	static EWorldBookmarkBrowserViewMode GetViewMode();
	static bool IsViewMode(EWorldBookmarkBrowserViewMode InViewMode);
	static void SetViewMode(EWorldBookmarkBrowserViewMode InViewMode);

private:
	static UWorldBookmarkBrowserSettings& Get();

	void ApplySettingsChanges();

	UPROPERTY(meta=(EditorConfig))
	bool bShowOnlyBookmarksForCurrentWorld = true;

	UPROPERTY(meta=(EditorConfig))
	bool bShowOnlyUncontrolledBookmarks = false;

	UPROPERTY(meta=(EditorConfig))
	bool bShowOnlyFavoriteBookmarks = false;

	UPROPERTY(meta=(EditorConfig))
	bool bShowOnlyLastRecentlyUsedBookmarks = false;

	UPROPERTY(meta=(EditorConfig))
	int32 MaxLastRecentlyUsedItems = 5;

	UPROPERTY(meta=(EditorConfig))
	TArray<FName> HiddenColumns;

	UPROPERTY(meta=(EditorConfig))
	EWorldBookmarkBrowserViewMode ViewMode;

	FWorldBookmarkBrowserSettingChanged OnSettingsChangedDelegate;

	static TObjectPtr<UWorldBookmarkBrowserSettings> Instance;
};

