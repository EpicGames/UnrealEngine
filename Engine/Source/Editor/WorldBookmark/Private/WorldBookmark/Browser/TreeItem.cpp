// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldBookmark/Browser/TreeItem.h"

namespace UE::WorldBookmark::Browser
{

ITreeItem::ITreeItem(ETreeItemType InType)
	: TreeItemType(InType)
	, ParentItem(nullptr)
{
}

ITreeItem::~ITreeItem()
{
}

ITreeItem* ITreeItem::GetParent() const
{
	return ParentItem;
}

void ITreeItem::SetParent(ITreeItem* InParent)
{
	ParentItem = InParent;
}

FString ITreeItem::GetAssetPath() const
{
	TStringBuilder<256> FullPathBuilder;

	const ITreeItem* CurrentItem = this;
	do
	{
		if (!CurrentItem->GetAssetName().IsNone())
		{
			FullPathBuilder.Prepend(CurrentItem->GetAssetName().ToString());
			FullPathBuilder.Prepend(TEXT("/"));
		}
	} while (CurrentItem = CurrentItem->GetParent(), CurrentItem);

	return FString(FullPathBuilder);
}

}
