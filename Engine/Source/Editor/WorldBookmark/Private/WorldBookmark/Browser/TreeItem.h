// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"

namespace UE::WorldBookmark::Browser
{

enum ETreeItemType
{
	Folder,
	WorldBookmark
};

// Define a common base class for your items
struct ITreeItem : public TSharedFromThis<ITreeItem>
{
public:
	/** Attempt to cast this item to another type if it is of that type. Returns nullptr if it fails */
	template <typename T>
	T* Cast()
	{
		return IsA<T>() ? StaticCast<T*>(this) : nullptr;
	}

	/** Attempt to cast this item to another type if it is of that type. Returns nullptr if it fails */
	template <typename T>
	const T* Cast() const
	{
		return IsA<T>() ? StaticCast<const T*>(this) : nullptr;
	}

	/** Returns true if this item is of the specified type */
	template <typename T>
	bool IsA() const
	{
		return TreeItemType == T::Type;
	}

	template <>
	bool IsA<ITreeItem>() const
	{
		return true;
	}

	virtual FName GetName() const = 0;

	virtual FName GetAssetName() const = 0;

	virtual bool CanRename() const = 0;

	virtual bool TryRename(FName InNewName, FText& OutErrorMessage) const = 0;
	
	virtual bool Rename(FName InNewName) = 0;

	virtual bool Delete() = 0;

	virtual FName GetIconName() const = 0;

	virtual FText GetText() const = 0;

	virtual FText GetTooltipText() const = 0;

	virtual void ShowInContentBrowser() const = 0;
	
	bool bInEditingMode = false;

	/** Delegate for when the context menu requests a rename */
	DECLARE_DELEGATE(FOnRenameRequested);
	FOnRenameRequested OnRenameRequested;

	/** Tree item type identifier */
	ETreeItemType TreeItemType;

	ITreeItem* GetParent() const;

	void SetParent(ITreeItem* InParent);

	FString GetAssetPath() const;

private:
	ITreeItem* ParentItem;

protected:
	ITreeItem(ETreeItemType InType);
	virtual ~ITreeItem();
};

typedef TSharedPtr<ITreeItem> FTreeItemPtr;
typedef TSharedRef<ITreeItem> FTreeItemRef;

}
