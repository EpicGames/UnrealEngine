// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldBookmark/WorldBookmark.h"
#include "EditorState/EditorStateSubsystem.h"
#include "EditorState/WorldEditorState.h"

#include "AssetRegistry/AssetRegistryHelpers.h"
#include "Editor.h"
#include "EngineUtils.h"
#include "FileHelpers.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Misc/MessageDialog.h"
#include "ScopedTransaction.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "WorldBookmark/WorldBookmarkEditorSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldBookmark)

DEFINE_LOG_CATEGORY(LogWorldBookmark);

#define LOCTEXT_NAMESPACE "WorldBookmark"

UWorldBookmark::UWorldBookmark(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UWorldBookmark::PostInitProperties()
{
	Super::PostInitProperties();

	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		BookmarkGuid = FGuid::NewGuid();
		BookmarkAssetPath = GetPathName();

		if (CanUpdate())
		{
			Update();
		}
	}
}

void UWorldBookmark::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	BookmarkGuid = FGuid::NewGuid();
	BookmarkAssetPath = GetPathName();
}

void UWorldBookmark::PostLoad()
{
	Super::PostLoad();

	check(BookmarkGuid.IsValid());
	BookmarkAssetPath = GetPathName();
}

void UWorldBookmark::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);
	SaveConfig();
}

void UWorldBookmark::OverridePerObjectConfigSection(FString& SectionName)
{
	SectionName = FString::Printf(TEXT("WorldBookmark %s"), *BookmarkGuid.ToString());
}

FName UWorldBookmark::GetWorldNameAssetTag()
{
	static const FName NAME_WorldName = "WorldName";
	return NAME_WorldName;
}

FName UWorldBookmark::GetCategoryAssetTag()
{
	static const FName NAME_WorldBookmarkCategory = "WorldBookmarkCategory";
	return NAME_WorldBookmarkCategory;
}

FSoftObjectPath UWorldBookmark::GetWorldFromAssetData(const FAssetData& InAssetData)
{
	FSoftObjectPath WorldPath;

	FString WorldAssetName;
	if (InAssetData.GetTagValue(GetWorldNameAssetTag(), WorldAssetName))
	{
		WorldPath = FSoftObjectPath(WorldAssetName);
		UAssetRegistryHelpers::FixupRedirectedAssetPath(WorldPath);
	}

	return WorldPath;
}

void UWorldBookmark::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	if (EditorState.HasState<UWorldEditorState>())
	{
		TSoftObjectPtr<UWorld> World = EditorState.GetStateChecked<UWorldEditorState>().GetStateWorld();
		if (!World.IsNull())
		{
			FSoftObjectPath WorldPath = FSoftObjectPath(World.ToString());
			UAssetRegistryHelpers::FixupRedirectedAssetPath(WorldPath);

			Context.AddTag(FAssetRegistryTag(GetWorldNameAssetTag(), WorldPath.ToString(), FAssetRegistryTag::TT_Hidden));
		}
	}
}

bool UWorldBookmark::CanLoad(FText* FailureReason) const
{
	if (!GIsEditor)
	{
		if (FailureReason)
		{
			*FailureReason = LOCTEXT("CanLoadFailure_NotInEditor", "Can't load bookmark outside of the editor");
		}

		return false;
	}

	if (GEditor->PlayWorld || GIsPlayInEditorWorld)
	{
		if (FailureReason)
		{
			*FailureReason = LOCTEXT("CanLoadFailure_IsInPIE", "Can't load bookmark while in PIE");
		}

		return false;
	}

	if (!HasEditorStates())
	{
		if (FailureReason)
		{
			*FailureReason = LOCTEXT("CanLoadFailure_IsUninitialized", "Can't load empty bookmark");
		}

		return false;
	}

	return true;
}

void UWorldBookmark::Load()
{
	LoadStates({});
}

void UWorldBookmark::LoadStates(const TArray<TSubclassOf<UEditorState>>& InStatesToLoad)
{
	FText CanLoadFailureReason;
	if (CanLoad(&CanLoadFailureReason))
	{
		UEditorStateSubsystem::Get()->RestoreEditorState(EditorState, InStatesToLoad);

		SetUserLastLoadedTimeStampUTC(FDateTime::UtcNow());
		SaveConfig();
	}
	else
	{
		if (GIsEditor)
		{
			FNotificationInfo Info(LOCTEXT("CantLoadBookmark", "Failed to load bookmark!"));
			Info.SubText = CanLoadFailureReason;
			Info.ExpireDuration = 5.0f;
			Info.bFireAndForget = true;
			Info.bUseLargeFont = false;
			Info.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Warning"));
			FSlateNotificationManager::Get().AddNotification(Info);
		}

		UE_LOGF(LogWorldBookmark, Error, "Failed to load bookmark: %ls", *CanLoadFailureReason.ToString());
	}
}

bool UWorldBookmark::CanUpdate(FText* FailureReason) const
{
	if (!GIsEditor)
	{
		if (FailureReason)
		{
			*FailureReason = LOCTEXT("CanUpdateFailure_NotInEditor", "Can't update bookmark outside of the editor");
		}

		return false;
	}

	UWorld* CurrentWorld = GEditor->GetEditorWorldContext().World();
	if (CurrentWorld == nullptr)
	{
		if (FailureReason)
		{
			*FailureReason = LOCTEXT("CanUpdateFailure_InvalidWorld", "Current world is invalid");
		}

		return false;
	}
	
	if (GEditor->PlayWorld || GIsPlayInEditorWorld)
	{
		if (FailureReason)
		{
			*FailureReason = LOCTEXT("CanUpdateFailure_IsInPIE", "Can't update bookmark while in PIE");
		}

		return false;
	}

	if (FPackageName::IsTempPackage(CurrentWorld->GetPackage()->GetName()))
	{
		if (FailureReason)
		{
			*FailureReason = LOCTEXT("CanUpdateFailure_UnsavedWorld", "Unsaved world");
		}

		return false;
	}

	return true;
}

void UWorldBookmark::Update()
{
	UpdateStates({});
}

void UWorldBookmark::UpdateStates(const TArray<TSubclassOf<UEditorState>>& InStatesToUpdate)
{
	FText CanUpdateFailureReason;
	if (CanUpdate(&CanUpdateFailureReason))
	{
		FScopedTransaction ScopedTransaction(LOCTEXT("UpdateWorldBookmarkTransaction", "Update World Bookmark"));

		Modify();

		UEditorStateSubsystem::Get()->CaptureEditorState(EditorState, InStatesToUpdate, this);

		PostEditChange();
	}
	else
	{
		if (GIsEditor)
		{
			FNotificationInfo Info(LOCTEXT("CantUpdateBookmark", "Failed to update bookmark!"));
			Info.SubText = CanUpdateFailureReason;
			Info.ExpireDuration = 5.0f;
			Info.bFireAndForget = true;
			Info.bUseLargeFont = false;
			Info.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Warning"));
			FSlateNotificationManager::Get().AddNotification(Info);
		}

		UE_LOGF(LogWorldBookmark, Error, "Failed to update bookmark: %ls", *CanUpdateFailureReason.ToString());
	}
}

bool UWorldBookmark::HasEditorStates() const
{
	return EditorState.HasStates();
}

bool UWorldBookmark::GetIsUserFavorite() const
{
	return bFavorite;
}

void UWorldBookmark::SetIsUserFavorite(bool bIsUserFavorite)
{
	if (bIsUserFavorite != bFavorite)
	{
		bFavorite = bIsUserFavorite;
		SaveConfig();
	}
}

FDateTime UWorldBookmark::GetUserLastLoadedTimeStampUTC() const
{
	return LastLoadedTimeStampUTC;
}

void UWorldBookmark::SetUserLastLoadedTimeStampUTC(const FDateTime& InLastLoadedTimeStampUTC)
{
	LastLoadedTimeStampUTC = InLastLoadedTimeStampUTC;
	SaveConfig();
}

const FWorldBookmarkCategory& UWorldBookmark::GetBookmarkCategory() const
{
	return UWorldBookmarkEditorSettings::GetCategory(CategoryGuid);
}

const FWorldBookmarkCategory FWorldBookmarkCategory::None = FWorldBookmarkCategory(NAME_None, FColor::Black);

FWorldBookmarkCategory::FWorldBookmarkCategory()
	: FWorldBookmarkCategory(TEXT("New Category"), FColor::MakeRandomSeededColor(GetTypeHash(FString(TEXT("New Category")))))
{
}

FWorldBookmarkCategory::FWorldBookmarkCategory(FName InName, FColor InColor)
	: Name(InName)
	, Color(InColor)
	, Guid(InName == NAME_None ? FGuid() : FGuid::NewGuid())
{
}

bool FWorldBookmarkCategory::operator<(const FWorldBookmarkCategory& Other) const
{
	if (Name.IsNone() || Other.Name.IsNone())
	{
		return Name.IsNone();
	}

	return Name.LexicalLess(Other.Name);
}

#undef LOCTEXT_NAMESPACE
