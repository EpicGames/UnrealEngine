// Copyright Epic Games, Inc. All Rights Reserved.
#include "WorldBookmark/WorldBookmarkCommands.h"
#include "WorldBookmark/WorldBookmarkStyle.h"

#define LOCTEXT_NAMESPACE "WorldBookmarkCommands"

FWorldBookmarkCommands::FWorldBookmarkCommands()
	: TCommands<FWorldBookmarkCommands>
	(
		"WorldBookmark",
		LOCTEXT("WorldBookmark", "World Bookmark"),
		NAME_None,
		FWorldBookmarkStyle::Get().GetStyleSetName()
	)
{}

void FWorldBookmarkCommands::RegisterCommands()
{
	UI_COMMAND(LoadBookmark, "Load Bookmark", "Load this bookmark and apply it's state to the world.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(UpdateBookmark, "Update Bookmark", "Update this bookmark using the current state of the world.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddToFavorite, "Add to Favorite", "Add this bookmark to your favorites.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveFromFavorite, "Remove from Favorite", "Remove this bookmark from your favorites.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(PlayFromLocation, "Play From Here", "Play from here.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(MoveCameraToLocation, "Move Camera Here", "Move the camera to the selected location.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(MoveBookmarkToNewFolder, "Move to New Folder", "Create a new folder containing the current selection.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CreateBookmarkInFolder, "Create Bookmark", "Create a new bookmark in the selected folder.", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
