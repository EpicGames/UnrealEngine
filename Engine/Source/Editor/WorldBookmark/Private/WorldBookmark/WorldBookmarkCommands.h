// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Framework/Commands/Commands.h"

class FWorldBookmarkCommands : public TCommands<FWorldBookmarkCommands>
{
public:
	FWorldBookmarkCommands();

	// Context Menu
	TSharedPtr<FUICommandInfo> LoadBookmark;
	TSharedPtr<FUICommandInfo> UpdateBookmark;
	TSharedPtr<FUICommandInfo> AddToFavorite;
	TSharedPtr<FUICommandInfo> RemoveFromFavorite;
	TSharedPtr<FUICommandInfo> PlayFromLocation;
	TSharedPtr<FUICommandInfo> MoveCameraToLocation;
	TSharedPtr<FUICommandInfo> MoveBookmarkToNewFolder;
	TSharedPtr<FUICommandInfo> CreateBookmarkInFolder;
	
	/**
	 * Initialize commands
	 */
	virtual void RegisterCommands() override;
};