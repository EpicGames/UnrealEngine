// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldBookmark/WorldBookmarkDetailsCustomization.h"
#include "WorldBookmark/WorldBookmark.h"
#include "WorldBookmark/WorldBookmarkEditorSettings.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "EditorState/WorldEditorState.h"
#include "IDetailChildrenBuilder.h"
#include "IStructureDetailsView.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "SPrimaryButton.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "WorldBookmarkDetailsCustomization"

static const FName NAME_NewCategory("New Category...");

FWorldBookmarkDetailsCustomization::FWorldBookmarkDetailsCustomization()
	: CachedDetailBuilder()
{
	GEditor->RegisterForUndo(this);

	OnWorldBookmarkEditorSettingsChangedHandle = UWorldBookmarkEditorSettings::OnSettingsChanged().AddRaw(this, &FWorldBookmarkDetailsCustomization::OnWorldBookmarkSettingsChanged);
}

FWorldBookmarkDetailsCustomization::~FWorldBookmarkDetailsCustomization()
{
	GEditor->UnregisterForUndo(this);

	UWorldBookmarkEditorSettings::OnSettingsChanged().Remove(OnWorldBookmarkEditorSettingsChangedHandle);
}

TSharedRef<IDetailCustomization> FWorldBookmarkDetailsCustomization::MakeInstance()
{
	return MakeShareable(new FWorldBookmarkDetailsCustomization);
}

void FWorldBookmarkDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	CachedDetailBuilder = &DetailBuilder;

	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	if (ObjectsBeingCustomized.Num() > 1)
	{
		return;
	}

	WorldBookmark = Cast<UWorldBookmark>(ObjectsBeingCustomized[0].Get());
	if (WorldBookmark == nullptr)
	{
		return;
	}

	RefreshBookmarkCategoriesList();

	// Customize the FGuid property
	TSharedRef<IPropertyHandle> GuidPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UWorldBookmark, CategoryGuid));
	IDetailPropertyRow* GuidPropertyRow = DetailBuilder.EditDefaultProperty(GuidPropertyHandle);
	
	GuidPropertyRow->CustomWidget()
		.NameContent()
		[
			GuidPropertyHandle->CreatePropertyNameWidget() // Use the default name widget
		]
		.ValueContent()
		.MinDesiredWidth(200)
		[
			SAssignNew(CategoriesComboBox, SComboBox<TSharedPtr<FWorldBookmarkCategory>>)
				.OptionsSource(&KnownCategories)
				.OnGenerateWidget(this, &FWorldBookmarkDetailsCustomization::MakeCategoryComboWidget)
				.OnSelectionChanged(this, &FWorldBookmarkDetailsCustomization::OnCategoryChanged)
				[
					SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.MaxWidth(20)
						[
							SNew(SBox)
								.Visibility(this, &FWorldBookmarkDetailsCustomization::GetCategoryColorVisibility)
								.Padding(0, 0, 4, 0)
								[
									SNew(SColorBlock)
										.Color(this, &FWorldBookmarkDetailsCustomization::GetCategoryColor)
										.CornerRadius(FVector4(4.0f, 4.0f, 4.0f, 4.0f))
										.Size(FVector2D(20, 16))
								]
						]

						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						[
							SNew(STextBlock)
								.Text(this, &FWorldBookmarkDetailsCustomization::GetCategoryText)
								.Font(IDetailLayoutBuilder::GetDetailFont())
						]
				]
		];

	// Sort state categories alphabetically
	TArray<UEditorState*> SortedEditorStateObjects = WorldBookmark->EditorState.GetStates();
	SortedEditorStateObjects.Sort([](UEditorState& A, UEditorState& B)
	{
		return A.GetCategoryText().CompareTo(B.GetCategoryText()) < 0;
	});

	for (UEditorState* EditorStateObject : SortedEditorStateObjects)
	{
		IDetailCategoryBuilder& EditorStateObjectCategory = DetailBuilder.EditCategory(EditorStateObject->GetFName(), EditorStateObject->GetCategoryText());
		EditorStateObjectCategory.AddExternalObjects({ EditorStateObject }, EPropertyLocation::Default, FAddPropertyParams().HideRootObjectNode(true).CreateCategoryNodes(false));
	}
}

void FWorldBookmarkDetailsCustomization::OnWorldBookmarkSettingsChanged(UObject* InSettingsObj, FPropertyChangedEvent& InEvent)
{
	RefreshCustomDetail();
}

void FWorldBookmarkDetailsCustomization::RefreshCustomDetail()
{
	if (CachedDetailBuilder)
	{
		RefreshBookmarkCategoriesList();

		CachedDetailBuilder->ForceRefreshDetails();
	}
}

void FWorldBookmarkDetailsCustomization::RefreshBookmarkCategoriesList()
{
	KnownCategories.Reset();

	// Add "None"
	KnownCategories.Emplace(MakeShared<FWorldBookmarkCategory>(FWorldBookmarkCategory{ NAME_None, FColor::Black }));
	
	// Add categories found in the project's settings
	for (const FWorldBookmarkCategory& Category : UWorldBookmarkEditorSettings::GetCategories())
	{
		KnownCategories.Emplace(MakeShared<FWorldBookmarkCategory>(Category));
	}

	// Add "New Category..."
	KnownCategories.Emplace(MakeShared<FWorldBookmarkCategory>(FWorldBookmarkCategory{ NAME_NewCategory, FColor::Black }));

	// Refresh the combo box
	if (CategoriesComboBox)
	{
		CategoriesComboBox->RefreshOptions();
	}
}

void FWorldBookmarkDetailsCustomization::PostUndo(bool bSuccess)
{
	// Refresh the UI
	RefreshCustomDetail();
}

void FWorldBookmarkDetailsCustomization::PostRedo(bool bSuccess)
{
	// Refresh the UI
	RefreshCustomDetail();
}

FLinearColor FWorldBookmarkDetailsCustomization::GetCategoryColor() const
{
	return WorldBookmark->GetBookmarkCategory().Color;
}

EVisibility FWorldBookmarkDetailsCustomization::GetCategoryColorVisibility() const
{
	return WorldBookmark->GetBookmarkCategory().Name.IsNone() ? EVisibility::Collapsed : EVisibility::Visible;
}

FText FWorldBookmarkDetailsCustomization::GetCategoryText() const
{
	return FText::FromName(WorldBookmark->GetBookmarkCategory().Name);
}

TSharedRef<SWidget> FWorldBookmarkDetailsCustomization::MakeCategoryComboWidget(TSharedPtr<FWorldBookmarkCategory> InItem)
{
	TSharedRef<SHorizontalBox> ItemWidget = SNew(SHorizontalBox);

	const bool bShowColorBlock = (InItem->Name != NAME_None) && (InItem->Name != NAME_NewCategory);
	if (bShowColorBlock)
	{
		ItemWidget->AddSlot()
			.MaxWidth(16)
			[
				SNew(SColorBlock)
					.Color(InItem->Color)
					.CornerRadius(FVector4(4.0f, 4.0f, 4.0f, 4.0f))
			];
	}

	ItemWidget->AddSlot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(bShowColorBlock ? 4 : 0, 0, 0, 0)
		[
			SNew(STextBlock)
				.Text(FText::FromName(InItem->Name))
				.Font(IDetailLayoutBuilder::GetDetailFont())
		];

	return ItemWidget;
}

void FWorldBookmarkDetailsCustomization::OnCategoryChanged(TSharedPtr<FWorldBookmarkCategory> NewSelection, ESelectInfo::Type SelectInfo)
{
	if (NewSelection.IsValid())
	{
		FScopedTransaction Transaction(LOCTEXT("ChangeBookmarkCategory", "Change Bookmark Category"));

		bool bShouldAssign = true;
		FWorldBookmarkCategory CategoryToAssign = *NewSelection;

		if (CategoryToAssign.Name == NAME_NewCategory)
		{
			bool bCreatedNewCategory = CreateNewCategory(CategoryToAssign);
			if (bCreatedNewCategory)
			{
				UWorldBookmarkEditorSettings::AddCategory(CategoryToAssign);
			}
			else
			{
				if (CachedDetailBuilder)
				{
					CachedDetailBuilder->ForceRefreshDetails();
				}
			}
			bShouldAssign = bCreatedNewCategory;
		}

		if (bShouldAssign && WorldBookmark->CategoryGuid != CategoryToAssign.Guid)
		{			
			WorldBookmark->Modify();
			WorldBookmark->CategoryGuid = CategoryToAssign.Guid;

			TSharedRef<IPropertyHandle> GuidPropertyHandle = CachedDetailBuilder->GetProperty(GET_MEMBER_NAME_CHECKED(UWorldBookmark, CategoryGuid));
			GuidPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		}
		else
		{
			Transaction.Cancel();
		}
	}
}

bool FWorldBookmarkDetailsCustomization::CreateNewCategory(FWorldBookmarkCategory& OutNewCategory)
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FStructureDetailsViewArgs StructureViewArgs;
	StructureViewArgs.bShowObjects = true;
	StructureViewArgs.bShowAssets = true;
	StructureViewArgs.bShowClasses = true;
	StructureViewArgs.bShowInterfaces = true;

	FDetailsViewArgs ViewArgs;
	ViewArgs.bAllowSearch = false;
	ViewArgs.bHideSelectionTip = false;
	ViewArgs.bShowObjectLabel = false;

	FWorldBookmarkCategory NewCategoryTemplate;
	NewCategoryTemplate.Color = FColor::MakeRandomColor();
	TSharedPtr<TStructOnScope<FWorldBookmarkCategory>> NewCategory;
	NewCategory = MakeShared<TStructOnScope<FWorldBookmarkCategory>>();
	NewCategory->InitializeAs<FWorldBookmarkCategory>(NewCategoryTemplate);

	TSharedPtr<IStructureDetailsView> DetailsView = PropertyEditorModule.CreateStructureDetailView(ViewArgs, StructureViewArgs, TSharedPtr<FStructOnScope>());

	// Set provided objects on SDetailsView
	DetailsView->SetStructureData(NewCategory);

	bool bCreatedNewCategory = false;

	const TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(LOCTEXT("NewWorldBookmarkCategoryTitle", "New World Bookmark Category"))
		.SizingRule(ESizingRule::Autosized)
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		.ClientSize(FVector2D(350, 450));

	Window->SetContent(
		SNew(SVerticalBox)

		+ SVerticalBox::Slot() // Add user input block
		.Padding(2, 2, 2, 4)
		[
			DetailsView->GetWidget().ToSharedRef()
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Bottom)
		.Padding(8.f, 16.f)
		[
			SNew(SUniformGridPanel)
				.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
				.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
				.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
				+ SUniformGridPanel::Slot(0, 0)
				[
					SNew(SPrimaryButton)
						.Text(LOCTEXT("Ok", "Ok"))
						.OnClicked_Lambda([&bCreatedNewCategory, Window]
						{
							bCreatedNewCategory = true;
							Window->RequestDestroyWindow();
							return FReply::Handled();
						})
				]
				+ SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
						.Text(LOCTEXT("Cancel", "Cancel"))
						.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
						.OnClicked_Lambda([&bCreatedNewCategory, Window]
						{
							bCreatedNewCategory = false;
							Window->RequestDestroyWindow();
							return FReply::Handled();
						})
				]
		]
	);

	GEditor->EditorAddModalWindow(Window);

	if (bCreatedNewCategory)
	{
		const bool bValidCategoryName = NewCategory->Get()->Name.ToString().ToLower() != FName(NAME_None).ToString().ToLower() &&
				                        NewCategory->Get()->Name.ToString().ToLower() != NAME_NewCategory.ToString().ToLower();
		bCreatedNewCategory = bValidCategoryName;
		if (bCreatedNewCategory)
		{
			OutNewCategory = FWorldBookmarkCategory(NewCategory->Get()->Name, NewCategory->Get()->Color);
		}		
	}
	
	return bCreatedNewCategory;
}

TSharedRef<IPropertyTypeCustomization> FWorldBookmarkCategoryCustomization::MakeInstance()
{
	return MakeShareable(new FWorldBookmarkCategoryCustomization);
}

void FWorldBookmarkCategoryCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	CachedStructPropertyHandle = InStructPropertyHandle;

	// Create the collapsed view with Name and Color
	InHeaderRow.NameContent()
	[
		InStructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(200.0f)
	[
		SNew(SHorizontalBox)
			
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.MaxWidth(20)
		[
			SNew(SBox)
			.Padding(0, 0, 4, 0)						
			[
				SNew(SColorBlock)
				.Color_Lambda([this]() { return GetEditedCategory().Color; })
				.CornerRadius(FVector4(4.0f, 4.0f, 4.0f, 4.0f))
				.Size(FVector2D(16, 16))
			]
		]

		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text_Lambda([this]() { return FText::FromName(GetEditedCategory().Name); })
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
	];
}

const FWorldBookmarkCategory& FWorldBookmarkCategoryCustomization::GetEditedCategory() const
{
	TArray<void*> RawData;
	CachedStructPropertyHandle->AccessRawData(RawData);
	FWorldBookmarkCategory* CurrentCategory = reinterpret_cast<FWorldBookmarkCategory*>(RawData[0]);
	return CurrentCategory ? *CurrentCategory : FWorldBookmarkCategory::None;
}

// Default behavior, add all children properties
void FWorldBookmarkCategoryCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		ChildBuilder.AddProperty(StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef());
	}
}

#undef LOCTEXT_NAMESPACE
