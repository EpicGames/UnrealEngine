// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "EditorUndoClient.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SWidget.h"

class UWorldBookmark;
struct FWorldBookmarkCategory;

/** UI customization for UWorldBookmark */
class FWorldBookmarkDetailsCustomization : public IDetailCustomization, public FEditorUndoClient
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	FWorldBookmarkDetailsCustomization();
	~FWorldBookmarkDetailsCustomization();

protected:
	//~ Begin IDetailCustomization Interface.
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization Interface.

	/** FEditorUndoClient interface */
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	/** End FEditorUndoClient interface */

	void RefreshCustomDetail();

private:
	TSharedRef<SWidget> MakeCategoryComboWidget(TSharedPtr<FWorldBookmarkCategory> InItem);

	void OnCategoryChanged(TSharedPtr<FWorldBookmarkCategory> NewSelection, ESelectInfo::Type SelectInfo);
	bool CreateNewCategory(FWorldBookmarkCategory& OutNewCategory);

	FLinearColor GetCategoryColor() const;
	EVisibility GetCategoryColorVisibility() const;
	FText GetCategoryText() const;

	void OnWorldBookmarkSettingsChanged(UObject* InSettings, FPropertyChangedEvent& InPropertyChangedEvent);
	
	void RefreshBookmarkCategoriesList();

	/** The detail builder for this cusomtomisation. */
	IDetailLayoutBuilder* CachedDetailBuilder;

	TArray<TSharedPtr<FWorldBookmarkCategory>> KnownCategories;
	TSharedPtr<SComboBox<TSharedPtr<FWorldBookmarkCategory>>> CategoriesComboBox;

	/** The currently edited bookmark */
	UWorldBookmark* WorldBookmark;

	FDelegateHandle OnWorldBookmarkEditorSettingsChangedHandle;
};

// UI customization for FWorldBookmarkCategory
class FWorldBookmarkCategoryCustomization : public IPropertyTypeCustomization
{
public:
	// Factory method to create an instance
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	// IPropertyTypeCustomization overrides
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

	const FWorldBookmarkCategory& GetEditedCategory() const;

	TSharedPtr<IPropertyHandle> CachedStructPropertyHandle;
};
