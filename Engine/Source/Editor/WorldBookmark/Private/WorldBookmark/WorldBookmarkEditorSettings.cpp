// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldBookmark/WorldBookmarkEditorSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldBookmarkEditorSettings)

UDeveloperSettings::FOnSettingsChanged& UWorldBookmarkEditorSettings::OnSettingsChanged()
{
	UWorldBookmarkEditorSettings& Settings = *GetMutableDefault<UWorldBookmarkEditorSettings>();
	return Settings.OnSettingChanged();
}

const FWorldBookmarkCategory& UWorldBookmarkEditorSettings::GetCategory(const FGuid& InCategoryGuid)
{
	const UWorldBookmarkEditorSettings& Settings = *GetDefault<UWorldBookmarkEditorSettings>();
	const FWorldBookmarkCategory* FoundCategory = Settings.Categories.FindByPredicate([&InCategoryGuid](const FWorldBookmarkCategory& InCategory)
	{
		return InCategory.Guid == InCategoryGuid;
	});

	if (FoundCategory)
	{
		return *FoundCategory;
	}

	return FWorldBookmarkCategory::None;
}

void UWorldBookmarkEditorSettings::AddCategory(const FWorldBookmarkCategory& InCategory)
{
	UWorldBookmarkEditorSettings* Settings = GetMutableDefault<UWorldBookmarkEditorSettings>();
	Settings->SetFlags(RF_Transactional); // Settings object is not transactional by default

	FWorldBookmarkCategory* ExistingCategory = Settings->Categories.FindByPredicate([&InCategory](const FWorldBookmarkCategory& Category)
	{
		return Category.Guid == InCategory.Guid;
	});

	bool bSettingsChanged = !ExistingCategory || (ExistingCategory->Name != InCategory.Name && ExistingCategory->Color != InCategory.Color);
	if (bSettingsChanged)
	{
		Settings->Modify();

		if (ExistingCategory)
		{
			ExistingCategory->Name = InCategory.Name;
			ExistingCategory->Color = InCategory.Color;
		}
		else
		{
			Settings->Categories.Add(InCategory);
		}

		Settings->PostEditChange();
	}
}

void UWorldBookmarkEditorSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	// Keep categories sorted by names
	Categories.StableSort([](const FWorldBookmarkCategory& LHS, const FWorldBookmarkCategory& RHS)
	{
		return LHS.Name.LexicalLess(RHS.Name);
	});

	Super::PostEditChangeProperty(PropertyChangedEvent);

	TryUpdateDefaultConfigFile();
}

const TArray<FWorldBookmarkCategory>& UWorldBookmarkEditorSettings::GetCategories()
{
	return GetDefault<UWorldBookmarkEditorSettings>()->Categories;
}

UWorldBookmarkEditorPerProjectUserSettings::UWorldBookmarkEditorPerProjectUserSettings()
{
	bEnableDefaultBookmarks = true;
	bEnableHomeBookmark = true;
}
