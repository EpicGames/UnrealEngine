// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Engine/DeveloperSettings.h"
#include "WorldBookmark/WorldBookmark.h"
#include "WorldBookmarkEditorSettings.generated.h"


UCLASS(config = Editor, DefaultConfig, meta = (DisplayName = "World Bookmark"))
class UWorldBookmarkEditorSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	DECLARE_MULTICAST_DELEGATE(UWorldBookmarkEditorSettingsChanged)

	static FOnSettingsChanged& OnSettingsChanged();

	static const FWorldBookmarkCategory& GetCategory(const FGuid& InCategoryGuid);
	static void AddCategory(const FWorldBookmarkCategory& InCategory);

	static const TArray<FWorldBookmarkCategory>& GetCategories();

protected:
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

private:
	UPROPERTY(config, EditAnywhere, Category = "Bookmark Categories", meta=(NoElementDuplicate, EditFixedOrder))
	TArray<FWorldBookmarkCategory> Categories;
};


UCLASS(config = EditorPerProjectUserSettings, meta = (DisplayName = "World Bookmark"))
class UWorldBookmarkEditorPerProjectUserSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UWorldBookmarkEditorPerProjectUserSettings();

	/** When enabled, the default bookmark will be applied when loading a level */
	UPROPERTY(config, EditAnywhere, Category="Default Level Bookmark")
	bool bEnableDefaultBookmarks;

	/** When enabled, the home bookmark will be applied when starting the editor without specifying a map */
	UPROPERTY(config, EditAnywhere, Category = "Home Bookmark")
	bool bEnableHomeBookmark;

	/** Bookmark to be applied when starting the editor without specifying a map */
	UPROPERTY(config, EditAnywhere, Category = "Home Bookmark", meta = (EditCondition = "bEnableHomeBookmark"))
	TSoftObjectPtr<UWorldBookmark> HomeBookmark;
};
