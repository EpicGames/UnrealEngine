// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldBookmark/WorldBookmarkFactory.h"
#include "WorldBookmark/WorldBookmark.h"
#include "Misc/AssertionMacros.h"
#include "Templates/SubclassOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldBookmarkFactory)


UWorldBookmarkFactory::UWorldBookmarkFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UWorldBookmark::StaticClass();

	bCreateNew = true;
	bEditAfterNew = true;
	bEditorImport = false;
}

UObject* UWorldBookmarkFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UWorldBookmark* Bookmark = NewObject<UWorldBookmark>(InParent, Name, Flags);
	check(Bookmark);
	return Bookmark;
}
