// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"

#include "WorldBookmarkFactory.generated.h"

class FFeedbackContext;
class UClass;
class UObject;

UCLASS()
class UWorldBookmarkFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	
};
