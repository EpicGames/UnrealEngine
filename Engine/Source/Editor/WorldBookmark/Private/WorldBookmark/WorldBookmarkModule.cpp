// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldBookmark/WorldBookmarkModule.h"
#include "WorldBookmark/WorldBookmark.h"
#include "WorldBookmark/WorldBookmarkCommands.h"
#include "WorldBookmark/WorldBookmarkDetailsCustomization.h"
#include "WorldBookmark/WorldBookmarkEditorSettings.h"
#include "WorldBookmark/WorldBookmarkStyle.h"
#include "WorldBookmark/Browser/SWorldBookmarkBrowser.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/AssetData.h"
#include "Editor.h"
#include "EditorState/EditorStateSubsystem.h"
#include "EditorState/WorldEditorState.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GameFramework/WorldSettings.h"
#include "HAL/PlatformApplicationMisc.h"
#include "JsonObjectConverter.h"
#include "LevelEditor.h"
#include "Misc/Base64.h"
#include "Misc/Compression.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "ObjectTools.h"
#include "PropertyEditorModule.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

IMPLEMENT_MODULE(FWorldBookmarkModule, WorldBookmark);

#define LOCTEXT_NAMESPACE "WorldBookmark"

const FName WorldBookmarkBrowserTabId("WorldBookmarkBrowser");

static FAutoConsoleCommand WorldBookmarkCaptureToStringCommand(
	TEXT("WorldBookmark.Capture"),
	TEXT("Capture the current state of the editor and log it to the console."),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		FString Str = FWorldBookmarkModule::CaptureToString();
		UE_LOGF(LogWorldBookmark, Display, "Restore the bookmark with 'WorldBookmark.Restore' and the following argument:\n%ls", *Str);
	})
);

static FAutoConsoleCommand WorldBookmarkCaptureToClipboardCommand(
	TEXT("WorldBookmark.CaptureToClipboard"),
	TEXT("Capture the current state of the editor and copy it to the clipboard."),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		FWorldBookmarkModule::CaptureToClipboard();
		UE_LOGF(LogWorldBookmark, Display, "WorldBookmark captured to clipboard");
	})
);

static FAutoConsoleCommand WorldBookmarkRestoreFromStringCommand(
	TEXT("WorldBookmark.Restore"),
	TEXT("Restore a bookmark from the text previously obtained from WorldBookmark.Capture."),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		TArray<FString> History;
		IConsoleManager::Get().GetConsoleHistory(TEXT(""), History);
		
		bool bRestoredFromConsoleHistory = false;

		// Try to retrieve a multiline command from the console history, as bookmarks strings may be pasted with newlines in them
		if (!History.IsEmpty())
		{
			FString BookmarkString = History.Last();
			if (BookmarkString.RemoveFromStart(TEXT("WorldBookmark.Restore ")))
			{
				bRestoredFromConsoleHistory = true;
				FWorldBookmarkModule::RestoreFromString(BookmarkString);
			}
		}
		
		// Failed to obtain the command from the console history
		// Fallback to the arguments
		if (!bRestoredFromConsoleHistory && Args.Num() == 1)
		{
			FWorldBookmarkModule::RestoreFromString(Args[0]);
		}
	})
);

static FAutoConsoleCommand WorldBookmarkRestoreFromClipboardCommand(
	TEXT("WorldBookmark.RestoreFromClipboard"),
	TEXT("Restore a bookmark from the clipboard."),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		FWorldBookmarkModule::RestoreFromClipboard();
	})
);

void FWorldBookmarkModule::StartupModule()
{
	FEditorDelegates::OnEditorBoot.AddLambda([this](double)
	{
		FWorldBookmarkCommands::Register();

		FLevelEditorModule& LevelEditorModule = FModuleManager::Get().LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		LevelEditorModule.OnRegisterTabs().AddRaw(this, &FWorldBookmarkModule::RegisterWorldBookmarkBrowserTab);

		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		ClassesToUnregisterOnShutdown.Add(UWorldBookmark::StaticClass()->GetFName());
		PropertyModule.RegisterCustomClassLayout(UWorldBookmark::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FWorldBookmarkDetailsCustomization::MakeInstance));

		StructsToUnregisterOnShutdown.Add(FWorldBookmarkCategory::StaticStruct()->GetFName());
		PropertyModule.RegisterCustomPropertyTypeLayout(FWorldBookmarkCategory::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FWorldBookmarkCategoryCustomization::MakeInstance));

		// When deleting a world, also delete associated bookmarks
		OnAddExtraObjectsToDeleteDelegateHandle = FEditorDelegates::OnAddExtraObjectsToDelete.AddRaw(this, &FWorldBookmarkModule::OnAddExtraObjectsToDelete);

		// Override the loading of the default startup map if the user specified a Home Bookmark
		OnEditorLoadDefaultStartupMapHandle = FEditorDelegates::OnEditorLoadDefaultStartupMap.AddRaw(this, &FWorldBookmarkModule::OnEditorLoadDefaultStartupMap);

		// Listen for map change events
		OnMapChangedHandle = LevelEditorModule.OnMapChanged().AddRaw(this, &FWorldBookmarkModule::OnMapChanged);

		// Validate level default bookmark changes
		OnDefaultBookmarkChangedHandle = AWorldSettings::OnDefaultBookmarkChanged.AddRaw(this, &FWorldBookmarkModule::OnDefaultBookmarkChanged);
	});
}

void FWorldBookmarkModule::ShutdownModule()
{
	if (GIsEditor && !IsRunningCommandlet())
	{
		if (FLevelEditorModule* LevelEditorModule = FModuleManager::Get().GetModulePtr<FLevelEditorModule>("LevelEditor"))
		{
			LevelEditorModule->OnRegisterTabs().RemoveAll(this);

			if (TSharedPtr<FTabManager> TabManager = LevelEditorModule->GetLevelEditorTabManager())
			{
				TabManager->UnregisterTabSpawner(WorldBookmarkBrowserTabId);
			}

			LevelEditorModule->OnMapChanged().Remove(OnMapChangedHandle);
		}		

		if (FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
		{
			for (const FName& ClassName : ClassesToUnregisterOnShutdown)
			{
				PropertyModule->UnregisterCustomClassLayout(ClassName);
			}

			for (const FName& StructName : StructsToUnregisterOnShutdown)
			{
				PropertyModule->UnregisterCustomPropertyTypeLayout(StructName);
			}
		}

		AWorldSettings::OnDefaultBookmarkChanged.Remove(OnDefaultBookmarkChangedHandle);

		ClassesToUnregisterOnShutdown.Empty();
		StructsToUnregisterOnShutdown.Empty();

		FEditorDelegates::OnAddExtraObjectsToDelete.Remove(OnAddExtraObjectsToDeleteDelegateHandle);
		FEditorDelegates::OnEditorLoadDefaultStartupMap.Remove(OnEditorLoadDefaultStartupMapHandle);
	}
}

void FWorldBookmarkModule::RegisterWorldBookmarkBrowserTab(TSharedPtr<FTabManager> InTabManager)
{
	const IWorkspaceMenuStructure& MenuStructure = WorkspaceMenu::GetMenuStructure();

	const FSlateIcon WorldPartitionIcon(FWorldBookmarkStyle::Get().GetStyleSetName(), "WorldBookmark.TabIcon");

	InTabManager->RegisterTabSpawner(WorldBookmarkBrowserTabId,
		FOnSpawnTab::CreateRaw(this, &FWorldBookmarkModule::SpawnWorldBookmarkBrowserTab))
		.SetDisplayName(LOCTEXT("WorldBookmarks", "World Bookmarks"))
		.SetTooltipText(LOCTEXT("WorldBookmarksTooltipText", "Open the World Bookmarks browser."))
		.SetGroup(MenuStructure.GetLevelEditorWorldPartitionCategory())
		.SetIcon(WorldPartitionIcon);
}

TSharedRef<SDockTab> FWorldBookmarkModule::SpawnWorldBookmarkBrowserTab(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> NewTab =
		SNew(SDockTab)
		.Label(LOCTEXT("WorldBookmarkBrowserTab", "World Bookmarks"))
		[
			CreateWorldBookmarkBrowser()
		];

	return NewTab;
}

TSharedRef<SWidget> FWorldBookmarkModule::CreateWorldBookmarkBrowser()
{
	return SNew(UE::WorldBookmark::Browser::SWorldBookmarkBrowser);
}

void FWorldBookmarkModule::OnAddExtraObjectsToDelete(const TArray<UObject*>& InObjectsToDelete, TSet<UObject*>& OutSecondaryObjects)
{
	// Gather a list of worlds
	TArray<const UWorld*> WorldsPendingDelete;
	for (const UObject* Object : InObjectsToDelete)
	{
		if (const UWorld* World = Cast<UWorld>(Object))
		{
			WorldsPendingDelete.Add(World);
		}
	}

	// If we are deleting worlds
	if (!WorldsPendingDelete.IsEmpty())
	{
		// Scan the asset registry and look for bookmarks for these worlds
		FARFilter ArFilter;
		ArFilter.ClassPaths.Add(UWorldBookmark::StaticClass()->GetClassPathName());
		ArFilter.bRecursiveClasses = true;
		for (const UWorld* WorldPendingDelete : WorldsPendingDelete)
		{
			ArFilter.TagsAndValues.Emplace(UWorldBookmark::GetWorldNameAssetTag(), WorldPendingDelete->GetPathName());
		}
		TArray<FAssetData> AssetsData;
		IAssetRegistry::GetChecked().GetAssets(ArFilter, AssetsData);

		// We've found some bookmarks related to these worlds, ask the user if they want to delete them
		if (!AssetsData.IsEmpty())
		{
			FTextBuilder TextBuilder;
			
			int32 NbLines = 0;
			const int32 MaxNbLines = 5;
			for (const FAssetData& AssetData : AssetsData)
			{
				if (++NbLines > MaxNbLines)
				{
					TextBuilder.AppendLineFormat(LOCTEXT("DeleteMatchingBookmarks_OverflowList", "    ({0} more assets...)"), FText::AsNumber(AssetsData.Num() - MaxNbLines));
					break;
				}

				TextBuilder.AppendLineFormat(LOCTEXT("DeleteMatchingBookmarks_AssetList", "    {0}"), FText::FromString(AssetData.PackageName.ToString()));
			}

			const FText AllAssetsText = TextBuilder.ToText();
			const FText MessageBoxTitle = LOCTEXT("DeleteMatchingBookmarks_Title", "Delete World Bookmark(s)?");
			const FText MessageBoxTextSingleAsset = FText::Format(LOCTEXT("DeleteMatchingBookmark_Text", "This world is referenced by a World Bookmark. Do you wish to delete this asset too?\n{0}"), AllAssetsText);
			const FText MessageBoxTextMultipleAssets = FText::Format(LOCTEXT("DeleteMatchingBookmarks_Text", "This world is referenced by {0} World Bookmarks. Do you wish to delete those assets too?\n{1}"), FText::AsNumber(AssetsData.Num()), AllAssetsText);

			const EAppReturnType::Type Response = FMessageDialog::Open(EAppMsgCategory::Info, EAppMsgType::YesNo, AssetsData.Num() > 1 ? MessageBoxTextMultipleAssets : MessageBoxTextSingleAsset, MessageBoxTitle);
			if (Response == EAppReturnType::Yes)
			{
				for (const FAssetData& AssetData : AssetsData)
				{
					if (UWorldBookmark* WorldBookmark = Cast<UWorldBookmark>(AssetData.GetAsset()))
					{
						OutSecondaryObjects.Add(WorldBookmark);
					}
				}
			}
		}
	}
}

void FWorldBookmarkModule::OnMapChanged(UWorld* InWorld, EMapChangeType InMapChangeType)
{
	if (InMapChangeType != EMapChangeType::LoadMap)
	{
		return;
	}

	// Load the default bookmark if the user has that option enabled
	const UWorldBookmarkEditorPerProjectUserSettings* Settings = GetDefault<UWorldBookmarkEditorPerProjectUserSettings>();
	if (Settings->bEnableDefaultBookmarks)
	{
		// Skip loading the default bookmark if the map change was actually caused by loading a bookmark
		if (!UEditorStateSubsystem::Get()->IsRestoringEditorState())
		{
			if (IsDefaultBookmarkValid(InWorld->GetWorldSettings()))
			{
				// At this point, make sure we are really dealing with a World Bookmark. Otherwise, do nothing.
				if (UWorldBookmark* WorldBookmark = Cast<UWorldBookmark>(InWorld->GetWorldSettings()->GetDefaultBookmark()))
				{
					UE_LOGF(LogWorldBookmark, Display, "Loading default bookmark %ls", *WorldBookmark->GetPathName());
					WorldBookmark->Load();
				}
			}
			else
			{					
				ShowInvalidDefaultBookmarkNotification(LOCTEXT("DefaultBookmarkIncorrectWorld_OnMapChanged", "Default bookmark not applied"));
			}
		}
	}
}

void FWorldBookmarkModule::OnDefaultBookmarkChanged(AWorldSettings* InWorldSettings)
{
	if (!IsDefaultBookmarkValid(InWorldSettings))
	{
		ShowInvalidDefaultBookmarkNotification(LOCTEXT("DefaultBookmarkIncorrectWorld_OnDefaultBookmarkChanged", "Invalid default bookmark"));
		InWorldSettings->SetDefaultBookmark(nullptr);
	}
}

bool FWorldBookmarkModule::IsDefaultBookmarkValid(const AWorldSettings* InWorldSettings) const
{
	UWorldBookmark* WorldBookmark = InWorldSettings ? Cast<UWorldBookmark>(InWorldSettings->GetDefaultBookmark()) : nullptr;
	if (!WorldBookmark)
	{
		return true; // Not specifying any bookmark (or, possibly, another type of bookmark) is valid
	}

	// Validate that the world bookmark's world is our actual world
	const UWorldEditorState* WorldEditorState = WorldBookmark->GetEditorState<UWorldEditorState>();
	TSoftObjectPtr<UWorld> BookmarkWorld = WorldEditorState ? WorldEditorState->GetStateWorld() : nullptr;
	return BookmarkWorld == InWorldSettings->GetWorld();
}

void FWorldBookmarkModule::ShowInvalidDefaultBookmarkNotification(const FText& InNotificationTitle) const
{
	// Show toast.
	FNotificationInfo Info(InNotificationTitle);
	Info.SubText = LOCTEXT("DefaultBookmarkIncorrectWorld_NotificationSubText", "The default bookmark is referencing another world");
	Info.ExpireDuration = 5.0f;
	Info.bFireAndForget = true;
	Info.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Warning"));
	Info.HyperlinkText = LOCTEXT("DefaultBookmarkIncorrectWorld_ShowWorldSettings", "Show World Settings");
	Info.Hyperlink = FSimpleDelegate::CreateLambda([=]()
	{
		// Open the World Settings tab
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		LevelEditorModule.GetLevelEditorTabManager()->TryInvokeTab(FName("WorldSettingsTab"));
	});

	FSlateNotificationManager::Get().AddNotification(Info);
}

void FWorldBookmarkModule::OnEditorLoadDefaultStartupMap(FCanLoadMap& InOutCanLoadDefaultStartupMap)
{
	const UWorldBookmarkEditorPerProjectUserSettings* Settings = GetDefault<UWorldBookmarkEditorPerProjectUserSettings>();
	if (Settings->bEnableHomeBookmark)
	{
		if (UWorldBookmark* HomeBookmark = Settings->HomeBookmark.LoadSynchronous())
		{
			UE_LOGF(LogWorldBookmark, Display, "Loading home bookmark %ls", *Settings->HomeBookmark.ToString());
			HomeBookmark->Load();

			// If the bookmark world was loaded successfully, prevent the loading of the default startup map.
			// Verify that the current map now matches the one from the bookmark
			if (const UWorldEditorState* WorldEditorState = HomeBookmark->GetEditorState<UWorldEditorState>())
			{
				TSoftObjectPtr<UWorld> BookmarkWorld = WorldEditorState->GetStateWorld();
				if (GEditor->GetEditorWorldContext().World() == BookmarkWorld)
				{
					InOutCanLoadDefaultStartupMap.SetFalse();
				}
			}
		}
	}
}

struct FBookmarkTextExport
{
	static TSharedPtr<FJsonValue> ExportPropertyToJSon(FProperty* Property, const void* Value)
	{
		static const UScriptStruct* BoxStruct = FindObject<UScriptStruct>(nullptr, TEXT("/Script/CoreUObject.Box"), EFindObjectFlags::ExactClass);
		static const UScriptStruct* VectorStruct = FindObject<UScriptStruct>(nullptr, TEXT("/Script/CoreUObject.Vector"), EFindObjectFlags::ExactClass);
		static const UScriptStruct* RotatorStruct = FindObject<UScriptStruct>(nullptr, TEXT("/Script/CoreUObject.Rotator"), EFindObjectFlags::ExactClass);

		if (FStructProperty* PropertyAsStruct = CastField<FStructProperty>(Property))
		{
			auto Round = [](double Val)
			{
				return FMath::RoundToInt(Val * 100) / 100.0;
			};

			if (PropertyAsStruct->Struct == BoxStruct)
			{
				FBox Box;
				PropertyAsStruct->CopySingleValue(&Box, Value);
				FString AsString = Box.IsValid ? FString::Printf(TEXT("%.2f %.2f %.2f %.2f %.2f %.2f"), Round(Box.Min.X), Round(Box.Min.Y), Round(Box.Min.Z), Round(Box.Max.X), Round(Box.Max.Y), Round(Box.Min.Z)) : TEXT("");
				return MakeShared<FJsonValueString>(*AsString);
			}
			else if (PropertyAsStruct->Struct == VectorStruct)
			{
				FVector Vec;
				PropertyAsStruct->CopySingleValue(&Vec, Value);
				FString AsString = !Vec.IsNearlyZero() ? FString::Printf(TEXT("%.2f %.2f %.2f"), Round(Vec.X), Round(Vec.Y), Round(Vec.Z)) : TEXT("");
				return MakeShared<FJsonValueString>(*AsString);
			}
			else if (PropertyAsStruct->Struct == RotatorStruct)
			{
				FRotator Rot;
				PropertyAsStruct->CopySingleValue(&Rot, Value);
				FString AsString = !Rot.IsNearlyZero() ? FString::Printf(TEXT("%.2f %.2f %.2f"), Round(Rot.Pitch), Round(Rot.Yaw), Round(Rot.Roll)) : TEXT("");
				return MakeShared<FJsonValueString>(*AsString);
			}
		}

		return nullptr;
	}

	static bool ImportPropertyFromJSon(const TSharedPtr<FJsonValue>& JsonValue, FProperty* Property, void* Value)
	{
		static const UScriptStruct* BoxStruct = FindObject<UScriptStruct>(nullptr, TEXT("/Script/CoreUObject.Box"), EFindObjectFlags::ExactClass);
		static const UScriptStruct* VectorStruct = FindObject<UScriptStruct>(nullptr, TEXT("/Script/CoreUObject.Vector"), EFindObjectFlags::ExactClass);
		static const UScriptStruct* RotatorStruct = FindObject<UScriptStruct>(nullptr, TEXT("/Script/CoreUObject.Rotator"), EFindObjectFlags::ExactClass);

		if (FStructProperty* PropertyAsStruct = CastField<FStructProperty>(Property))
		{
			TArray<double> DoubleArray;

			auto ToDoubleArray = [](const FString& InString, TArray<double>& OutDoubles) -> int
			{
				TArray<FString> StringArray;
				InString.ParseIntoArray(StringArray, TEXT(" "));
				Algo::Transform(StringArray, OutDoubles, [](const FString& String)
				{
					return FCString::Atod(*String);
				});
				return OutDoubles.Num();
			};

			if (PropertyAsStruct->Struct == BoxStruct)
			{
				FBox& Box = *static_cast<FBox*>(Value);
				if (ToDoubleArray(JsonValue->AsString(), DoubleArray) == 6)
				{
					FMemory::Memcpy(Value, DoubleArray.GetData(), 6 * sizeof(double));
					Box.IsValid = true;
				}
				else
				{
					Box.IsValid = false;
				}
				return true;
			}
			else if ((PropertyAsStruct->Struct == VectorStruct) || (PropertyAsStruct->Struct == RotatorStruct))
			{
				if (ToDoubleArray(JsonValue->AsString(), DoubleArray) == 3)
				{					
					FMemory::Memcpy(Value, DoubleArray.GetData(), 3 * sizeof(double));
				}
				else
				{
					FMemory::Memzero(Value, 3 * sizeof(double));
				}
				return true;
			}
		}

		return false;
	}

	int8  Version = 0;						// For possible future expansions
	int32 UncompressedSize = 0;
	TArray<uint8> CompressedData;
};


static FString FormatBookmarkString(const FString& InBookmarkString, const int32 InMaxLineLength)
{
	FString Result;
	int32 InputLength = InBookmarkString.Len();
	int32 CurrentIndex = 0;

	// Process the input string in chunks of 100 characters
	while (CurrentIndex < InputLength)
	{
		// Determine the length of the next chunk
		int32 RemainingCharacters = InputLength - CurrentIndex;
		int32 ChunkLength = FMath::Min(RemainingCharacters, InMaxLineLength);

		// Extract the chunk
		FString Chunk = InBookmarkString.Mid(CurrentIndex, ChunkLength);
		Result += Chunk;

		// Add a newline if we're not at the end of the string
		CurrentIndex += ChunkLength;
		if (CurrentIndex < InputLength)
		{
			Result += TEXT("\n");
		}
	}

	// Check if the last line needs padding
	int32 LastLineStart = Result.Find(TEXT("\n"), ESearchCase::IgnoreCase, ESearchDir::FromEnd) + 1;
	if (LastLineStart == 0)
	{
		LastLineStart = 0; // No newlines, start at the beginning
	}

	int32 LastLineLength = Result.Len() - LastLineStart;
	if (LastLineLength < InMaxLineLength)
	{
		Result += FString::ChrN(InMaxLineLength - LastLineLength, TEXT('='));
	}

	return Result;
}

FString FWorldBookmarkModule::CaptureToString()
{
	// Capture the editor state
	FEditorStateCollection EditorStateCollection;
	UEditorStateSubsystem::Get()->CaptureEditorState(EditorStateCollection, GetTransientPackage());
	
	// Convert to a Json string
	FString BookmarkAsJson;
	FJsonObjectConverter::CustomExportCallback CustomExportCallback;
	CustomExportCallback.BindStatic(FBookmarkTextExport::ExportPropertyToJSon);
	FJsonObjectConverter::UStructToJsonObjectString(EditorStateCollection, BookmarkAsJson, 0, CPF_Deprecated | CPF_Transient | CPF_DuplicateTransient | CPF_TextExportTransient, 0, &CustomExportCallback, false);
	
	// Convert string to UTF8
	const FTCHARToUTF8 BufferString(*BookmarkAsJson);

	FBookmarkTextExport BookmarkTextExport;
	BookmarkTextExport.UncompressedSize = BufferString.Length();
	BookmarkTextExport.CompressedData.SetNumUninitialized(BookmarkTextExport.UncompressedSize);

	// Compress the UTF8 Json string using Zlib
	int32 CompressedSize = BufferString.Length();
	FCompression::CompressMemory(NAME_Zlib, BookmarkTextExport.CompressedData.GetData(), CompressedSize, BufferString.Get(), BufferString.Length(), COMPRESS_BiasSize);
	BookmarkTextExport.CompressedData.SetNum(CompressedSize);

	// Write all the info to a buffer
	TArray<uint8> ExportBuffer;
	FMemoryWriter MemoryWriter(ExportBuffer);
	MemoryWriter << BookmarkTextExport.Version;
	MemoryWriter << BookmarkTextExport.UncompressedSize;
	MemoryWriter << BookmarkTextExport.CompressedData;

	// Encode buffer to a base64 string
	FString Result = FString("BM") + FBase64::Encode(ExportBuffer);

	const int32 MAX_LINE_LENGTH = 100;
	return FormatBookmarkString(Result, MAX_LINE_LENGTH);
}

void FWorldBookmarkModule::CaptureToClipboard()
{
	FString BookmarkAsString = CaptureToString();
	FPlatformApplicationMisc::ClipboardCopy(*BookmarkAsString);
}

bool FWorldBookmarkModule::RestoreFromString(const FString& InBookmarkAsString)
{
	auto ShowInvalidBookmarkDataNotification = []()
	{
		// Show toast.
		FNotificationInfo Info(LOCTEXT("BookmarkFromString_Invalid_Text", "Invalid bookmark data"));
		Info.SubText = LOCTEXT("BookmarkFromString_Invalid_SubText", "The bookmark was not restored.");
		Info.ExpireDuration = 3.0f;
		Info.bFireAndForget = true;
		Info.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Error"));
		FSlateNotificationManager::Get().AddNotification(Info);
	};

	if (!InBookmarkAsString.StartsWith("BM"))
	{
		UE_LOGF(LogWorldBookmark, Error, "RestoreFromString: Invalid bookmark string");
		ShowInvalidBookmarkDataNotification();
		return false;
	}

	// Remove 'BM' header and any newlines
	FString Base64String = InBookmarkAsString.RightChop(2);
	Base64String.ReplaceInline(TEXT("\n"), TEXT(""));
	Base64String.ReplaceInline(TEXT("\r"), TEXT(""));
	Base64String.ReplaceInline(TEXT(" "), TEXT(""));
	
	// Decode base64 string to a buffer
	TArray<uint8> ImportBuffer;
	if (!FBase64::Decode(Base64String, ImportBuffer))
	{
		UE_LOGF(LogWorldBookmark, Error, "RestoreFromString: Failed to decode base64 bookmark data");
		ShowInvalidBookmarkDataNotification();
		return false;
	}

	FMemoryReader MemoryReader(ImportBuffer);

	// Read all the info from the buffer
	FBookmarkTextExport BookmarkTextImport;
	if (MemoryReader.TotalSize() > sizeof(FBookmarkTextExport))
	{
		MemoryReader << BookmarkTextImport.Version;
		MemoryReader << BookmarkTextImport.UncompressedSize;
		MemoryReader << BookmarkTextImport.CompressedData;
	}
	else
	{
		UE_LOGF(LogWorldBookmark, Error, "RestoreFromString: Failed to read compressed bookmark data");
		ShowInvalidBookmarkDataNotification();
		return false;
	}
	
	// Decompress the buffer to a UTF8 Json string
	TArray<uint8> UncompressedData;
	UncompressedData.SetNum(BookmarkTextImport.UncompressedSize);
	if (!FCompression::UncompressMemory(NAME_Zlib, UncompressedData.GetData(), BookmarkTextImport.UncompressedSize, BookmarkTextImport.CompressedData.GetData(), BookmarkTextImport.CompressedData.Num(), COMPRESS_BiasSize))
	{
		UE_LOGF(LogWorldBookmark, Error, "RestoreFromString: Failed to uncompress bookmark data");
		ShowInvalidBookmarkDataNotification();
		return false;
	}

	// Convert UTF8 Json string to an FString
	FUTF8ToTCHAR BufferToString((const ANSICHAR*)UncompressedData.GetData(), UncompressedData.Num());
	FString BookmarkAsJson = FString::ConstructFromPtrSize(BufferToString.Get(), BufferToString.Length());;
	
	// Read the editor state from the Json
	FEditorStateCollectionGCObject EditorStateCollectionGCObject;
	FJsonObjectConverter::CustomImportCallback CustomImportCallback;
	CustomImportCallback.BindStatic(FBookmarkTextExport::ImportPropertyFromJSon);
	if (!FJsonObjectConverter::JsonObjectStringToUStruct(BookmarkAsJson, &EditorStateCollectionGCObject.EditorStateCollection, 0, CPF_Deprecated | CPF_Transient | CPF_DuplicateTransient | CPF_TextExportTransient, false, nullptr, &CustomImportCallback))
	{
		UE_LOGF(LogWorldBookmark, Error, "RestoreFromString: Failed to restore bookmark data");
		ShowInvalidBookmarkDataNotification();
		return false;
	}
	
	// Restore the editor state
	UEditorStateSubsystem::Get()->RestoreEditorState(EditorStateCollectionGCObject.EditorStateCollection);

	return true;
}

bool FWorldBookmarkModule::RestoreFromClipboard()
{
	FString BookmarkAsString;
	FPlatformApplicationMisc::ClipboardPaste(BookmarkAsString);
	return RestoreFromString(BookmarkAsString);
}

#undef LOCTEXT_NAMESPACE
