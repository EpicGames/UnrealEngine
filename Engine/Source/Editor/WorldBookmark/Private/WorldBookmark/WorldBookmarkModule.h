// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
#include "UnrealEdMisc.h"

class FSpawnTabArgs;
class FTabManager;
class SDockTab;
class SWidget;

/**
 * Module dedicated to the World Bookmark feature
 */
class FWorldBookmarkModule : public IModuleInterface
{
public:
	/**
	 * Called right after the module DLL has been loaded and the module object has been created
	 */
	virtual void StartupModule();

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 */
	virtual void ShutdownModule();

    /** 
     * Capture a bookmark as a string, can be restored with RestoreFromString().
	 */
	static FString CaptureToString();

    /** 
     * Capture a bookmark to the clipboard, can be restored with RestoreFromClipboard().
	 */
	static void CaptureToClipboard();

    /** 
     * Restore a bookmark from the provided string.
	 */
	static bool RestoreFromString(const FString& InBookmarkAsString);

    /** 
     * Restore a bookmark from the clipboard.
	 */
	static bool RestoreFromClipboard();

private:
	void RegisterWorldBookmarkBrowserTab(TSharedPtr<FTabManager> InTabManager);
	TSharedRef<SDockTab> SpawnWorldBookmarkBrowserTab(const FSpawnTabArgs& Args);
	TSharedRef<SWidget> CreateWorldBookmarkBrowser();

	void OnAddExtraObjectsToDelete(const TArray<UObject*>& InObjectsToDelete, TSet<UObject*>& OutSecondaryObjects);
	void OnMapChanged(UWorld* InWorld, EMapChangeType InMapChangeType);
	void OnDefaultBookmarkChanged(AWorldSettings* InWorldSettings);
	void OnEditorLoadDefaultStartupMap(FCanLoadMap& InOutCanLoadDefaultStartupMap);

	bool IsDefaultBookmarkValid(const AWorldSettings* InWorldSettings) const;
	void ShowInvalidDefaultBookmarkNotification(const FText& InNotificationTitle) const;

	FDelegateHandle OnAddExtraObjectsToDeleteDelegateHandle;
	FDelegateHandle OnMapChangedHandle;
	FDelegateHandle OnDefaultBookmarkChangedHandle;
	FDelegateHandle OnEditorLoadDefaultStartupMapHandle;

	TArray<FName> ClassesToUnregisterOnShutdown;
	TArray<FName> StructsToUnregisterOnShutdown;
};
