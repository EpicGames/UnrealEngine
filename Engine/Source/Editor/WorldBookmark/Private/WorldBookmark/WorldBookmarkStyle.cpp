// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldBookmark/WorldBookmarkStyle.h"

#include "Styling/SlateStyleMacros.h"

#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"

FWorldBookmarkStyle::FWorldBookmarkStyle()
	: FSlateStyleSet("WorldBookmark")
{
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);

	SetParentStyleName(FAppStyle::GetAppStyleSetName());

	SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	// Asset
	Set("ClassIcon.WorldBookmark", new IMAGE_BRUSH_SVG("Starship/AssetIcons/WorldBookmark_16", Icon16x16));
	Set("ClassThumbnail.WorldBookmark", new IMAGE_BRUSH_SVG("Starship/AssetIcons/WorldBookmark_64", Icon64x64));

	// Commands
	Set("WorldBookmark.LoadBookmark", new IMAGE_BRUSH_SVG("Starship/Common/NextArrow", Icon16x16));
	Set("WorldBookmark.UpdateBookmark", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Update", Icon20x20));
	Set("WorldBookmark.CreateBookmark", new CORE_IMAGE_BRUSH_SVG("Starship/Common/plus", Icon16x16, FStyleColors::AccentGreen));
	Set("WorldBookmark.AddToFavorite", new IMAGE_BRUSH("Icons/Star_16x", Icon16x16));
	Set("WorldBookmark.RemoveFromFavorite", new IMAGE_BRUSH("Icons/EmptyStar_16x", Icon16x16));
	Set("WorldBookmark.PlayFromLocation", new IMAGE_BRUSH_SVG("Starship/Common/play", Icon20x20));
	Set("WorldBookmark.MoveCameraToLocation", new IMAGE_BRUSH_SVG("Starship/EditorViewport/actor-pilot-camera", Icon16x16));
	Set("WorldBookmark.MoveBookmarkToNewFolder", new CORE_IMAGE_BRUSH_SVG("Starship/Common/folder-plus", Icon16x16));
	Set("WorldBookmark.CreateBookmarkInFolder", new IMAGE_BRUSH_SVG("Starship/AssetIcons/WorldBookmark_16", Icon16x16));

	// Icons
	Set("WorldBookmark.TabIcon", new IMAGE_BRUSH_SVG("Starship/Common/Bookmarks", Icon16x16));
	Set("WorldBookmark.IsFavorite", new IMAGE_BRUSH("Icons/Star_16x", Icon16x16));
	Set("WorldBookmark.IsNotFavorite", new IMAGE_BRUSH("Icons/EmptyStar_16x", Icon16x16));
	Set("WorldBookmark.RecentlyUsed", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Recent", Icon20x20));
	Set("WorldBookmark.FolderClosed", new CORE_IMAGE_BRUSH_SVG("Starship/Common/folder-closed", Icon16x16, FStyleColors::AccentFolder));
	Set("WorldBookmark.FolderOpen", new CORE_IMAGE_BRUSH_SVG("Starship/Common/folder-open", Icon16x16, FStyleColors::AccentFolder));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FWorldBookmarkStyle::~FWorldBookmarkStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

FWorldBookmarkStyle& FWorldBookmarkStyle::Get()
{
	static FWorldBookmarkStyle Inst;
	return Inst;
}
