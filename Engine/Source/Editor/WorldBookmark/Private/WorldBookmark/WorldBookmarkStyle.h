// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyle.h"

class FWorldBookmarkStyle final : public FSlateStyleSet
{
public:
	FWorldBookmarkStyle();
	virtual ~FWorldBookmarkStyle();

	static FWorldBookmarkStyle& Get();
};
