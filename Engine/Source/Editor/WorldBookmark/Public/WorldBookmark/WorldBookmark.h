// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "UObject/ObjectSaveContext.h"
#include "EditorState/EditorStateCollection.h"
#include "Engine/BookmarkBase.h"
#include "WorldBookmark.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogWorldBookmark, Log, All);

/**
 * World Bookmark Category
 */
USTRUCT()
struct FWorldBookmarkCategory
{
	GENERATED_BODY()

	static const FWorldBookmarkCategory None;

	FWorldBookmarkCategory();
	FWorldBookmarkCategory(FName InName, FColor InColor);

	bool operator<(const FWorldBookmarkCategory& Other) const;

	UPROPERTY(EditAnywhere, Category=Category)
	FName Name;

	UPROPERTY(EditAnywhere, Category=Category, Meta=(HideAlphaChannel, DontUpdateWhileEditing, IgnoreForMemberInitializationTest))
	FColor Color;

    UPROPERTY(Meta=(IgnoreForMemberInitializationTest))
	FGuid Guid;
};

/**
 * World Bookmarks are assets that stores the state of the editor world.
 */
UCLASS(MinimalAPI, Config = UserWorldBookmarks, PerObjectConfig)
class UWorldBookmark : public UBookmarkBase
{
	GENERATED_UCLASS_BODY()

	/** Test whether it's possible to load this bookmark given the current state of the editor. */
	bool CanLoad(FText* FailureReason = nullptr) const;

	/** Test whether it's possible to update this bookmark given the current state of the editor. */
	bool CanUpdate(FText* FailureReason = nullptr) const;

	/** Load the bookmark data, restoring the editor state (loaded world, camera location, etc) to what is defined by the bookmark. */
	void Load();

	/** Update the bookmark so that it reflects the current state of the editor. */
	void Update();

	/** Load the bookmark data, restoring only a specific set of editor states. */
	void LoadStates(const TArray<TSubclassOf<UEditorState>>& InStatesToLoad);

	/** Update a set of states for the bookmark. */
	void UpdateStates(const TArray<TSubclassOf<UEditorState>>& InStatesToUpdate);

	/** Return true if the world bookmark contains states that can be restored. */
	bool HasEditorStates() const;

	/** Return true if this bookmark was flagged as being a favorite bookmark of the user. */
	bool GetIsUserFavorite() const;

	/** Mark this bookmark as being a favorite bookmark of the user. */
	void SetIsUserFavorite(bool bIsUserFavorite);

	/** Retrieve the last time that bookmark was loaded, in UTC. */
	FDateTime GetUserLastLoadedTimeStampUTC() const;

	/** Store the last time that bookmark was loaded, in UTC. */
	void SetUserLastLoadedTimeStampUTC(const FDateTime& InLastLoadedTimeStampUTC);

	/** Retrieve the world bookmark category for this bookmark. */
	const FWorldBookmarkCategory& GetBookmarkCategory() const;

    /** Get the asset registry tag used to store the matching world's name. */
	static FName GetWorldNameAssetTag();

	/** Get the asset registry tag used to store bookmark category info. */
	static FName GetCategoryAssetTag();

	/** Get the world associated with a bookmark's asset data. */
	static FSoftObjectPath GetWorldFromAssetData(const FAssetData& InAssetData);

	template <typename TEditorStateType>
	typename TEnableIf<TIsDerivedFrom<TEditorStateType, UEditorState>::IsDerived, bool>::Type HasEditorState() const
	{
		return EditorState.HasState<TEditorStateType>();
	}

	template <typename TEditorStateType>
	typename TEnableIf<TIsDerivedFrom<TEditorStateType, UEditorState>::IsDerived, const TEditorStateType*>::Type GetEditorState() const
	{
		return EditorState.GetState<TEditorStateType>();
	}

private:
	friend class FWorldBookmarkDetailsCustomization;

	//~ Begin UObject Interface
	virtual void PostLoad() override;
	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	virtual void PostInitProperties() override;
	virtual void PostDuplicate(bool bDuplicateForPIE) override;
	virtual void OverridePerObjectConfigSection(FString& SectionName) override;
	//~ End UObject Interface
		
private:
	// State of the editor.
	UPROPERTY()
	FEditorStateCollection EditorState;

	UPROPERTY(EditAnywhere, Category=Category, meta=(DisplayName="Category"))
	FGuid CategoryGuid;

	// BookmarkGuid is our key to fetch the user settings for this bookmark in the Bookmarks.ini config file.
	// It will remain unique even if redirectors are created for the world or the bookmark itself.
	UPROPERTY(DuplicateTransient)
	FGuid BookmarkGuid;

	// BEGIN - User settings saved to the config
	
	// Last loaded time (UTC)
	UPROPERTY(Config, Transient)
	FDateTime LastLoadedTimeStampUTC;

	// User favorite
	UPROPERTY(Config, Transient)
	bool bFavorite;

	// Unused - The sole purpose of this property is to help users who would want to investigate/make changes to the ini themselves, as GUIDs are pretty opaque.
	UPROPERTY(Config, Transient)
	FString BookmarkAssetPath;

	// END - User settings saved to the config
};
