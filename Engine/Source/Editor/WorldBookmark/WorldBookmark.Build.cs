// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class WorldBookmark : ModuleRules
{
	public WorldBookmark(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ApplicationCore",
				"AssetDefinition",
				"AssetRegistry",
				"AssetTools",
				"ContentBrowser",
				"ContentBrowserData",
				"Core",
				"CoreUObject",
				"DeveloperSettings",
				"EditorSubsystem",
				"EditorConfig",
				"Engine",
				"InputCore",
				"Json",
				"JsonUtilities",
				"LevelEditor",
				"Projects",
				"PropertyEditor",
				"SceneOutliner",
				"SourceControl",
				"Slate",
				"SlateCore",				
				"ToolMenus",
				"ToolWidgets",
				"UncontrolledChangelists",
				"UnrealEd",
				"WorkspaceMenuStructure"
			}
		);
	}
}
